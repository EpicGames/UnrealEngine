[CoreRedirects]
+PropertyRedirects=(OldName="Workspace.Assets",NewName="Workspace.Assets_DEPRECATED")