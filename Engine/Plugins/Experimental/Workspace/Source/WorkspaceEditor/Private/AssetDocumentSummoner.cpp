// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDocumentSummoner.h"

#include "WorkspaceEditor.h"
#include "AssetDefinitionRegistry.h"
#include "ClassIconFinder.h"
#include "StructUtils/InstancedStruct.h"
#include "WorkspaceEditorModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceDocumentState.h"
#include "SWorkspaceTabWrapper.h"
#include "WorkspaceTabPayload.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDocumentSummoner"

namespace UE::Workspace
{

const FName FTabPayload_WorkspaceDocument::DocumentPayloadName(TEXT("WorkspaceDocumentPayload"));

FAssetDocumentSummoner::FAssetDocumentSummoner(FName InIdentifier, TSharedPtr<FWorkspaceEditor> InHostingApp, bool bInAllowUnsupportedClasses /*= false*/)
	: FDocumentTabFactory(InIdentifier, InHostingApp)
	, HostingAppPtr(InHostingApp)
	, bAllowUnsupportedClasses(bInAllowUnsupportedClasses)
{
}

void FAssetDocumentSummoner::SetAllowedClassPaths(TConstArrayView<FTopLevelAssetPath> InAllowedClassPaths)
{
	AllowedClassPaths = InAllowedClassPaths;
}

void FAssetDocumentSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SWorkspaceTabWrapper> TabWrapper = StaticCastSharedRef<SWorkspaceTabWrapper>(Tab->GetContent());
	if (TabWrapper->GetDocumentObject().Get())
	{
		HostingAppPtr.Pin()->SetFocusedDocument(TabWrapper->GetDocument());
	}
}

void FAssetDocumentSummoner::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
}

void FAssetDocumentSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
}

void FAssetDocumentSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	if(TSharedPtr<FWorkspaceEditor> WorkspaceEditor = HostingAppPtr.Pin())
	{
		if(UObject* Object = Payload->IsValid() ? FTabPayload_WorkspaceDocument::CastChecked<UObject>(Payload.ToSharedRef()) : nullptr)
		{
			TSharedRef<SWorkspaceTabWrapper> TabWrapper = StaticCastSharedRef<SWorkspaceTabWrapper>(Tab->GetContent());
			FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
			const FWorkspaceOutlinerItemExport& Export = FTabPayload_WorkspaceDocument::GetExport(Payload.ToSharedRef());
			const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, Object});
			if(DocumentArgs && DocumentArgs->OnGetDocumentState.IsBound())
			{
				WorkspaceEditor->RecordDocumentState(DocumentArgs->OnGetDocumentState.Execute(FWorkspaceEditorContext(WorkspaceEditor.ToSharedRef(), {Export, Object}), TabWrapper->GetContent()));
			}
			else
			{
				WorkspaceEditor->RecordDocumentState(TInstancedStruct<FWorkspaceDocumentState>::Make(Object, FTabPayload_WorkspaceDocument::GetExport(Payload.ToSharedRef())));
			}
		}
	}
}

TAttribute<FText> FAssetDocumentSummoner::ConstructTabName(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWorkspaceEditor> WorkspaceEditor = HostingAppPtr.Pin();
	UObject* DocumentID = FTabPayload_WorkspaceDocument::CastChecked<UObject>(Info.Payload.ToSharedRef());
	if(!WorkspaceEditor.IsValid() || DocumentID == nullptr)
	{
		return LOCTEXT("NoneObjectName", "None");
	}

	FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
	const FWorkspaceOutlinerItemExport& Export = FTabPayload_WorkspaceDocument::GetExport(Info.Payload.ToSharedRef());
	if(const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, DocumentID }))
	{
		if(DocumentArgs->OnGetTabName.IsBound())
		{
			return DocumentArgs->OnGetTabName.Execute(FWorkspaceEditorContext(WorkspaceEditor.ToSharedRef(), DocumentID, Export));
		}
	}

	return MakeAttributeLambda([WeakObject = TWeakObjectPtr<UObject>(DocumentID)]()
	{
		if(UObject* Object = WeakObject.Get())
		{
			return FText::FromName(Object->GetFName());
		}

		return LOCTEXT("UnknownObjectName", "Unknown");
	});
}

bool FAssetDocumentSummoner::IsPayloadSupported(TSharedRef<FTabPayload> Payload) const
{
	if(UObject* Object = FTabPayload_WorkspaceDocument::CastChecked<UObject>(Payload))
	{
		FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
		if(const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({FTabPayload_WorkspaceDocument::GetExport(Payload), Object }))
		{
			return AllowedClassPaths.Contains(Object->GetClass()->GetClassPathName()) || bAllowUnsupportedClasses;
		}
	}

	return false;
}

TAttribute<FText> FAssetDocumentSummoner::ConstructTabLabelSuffix(const FWorkflowTabSpawnInfo& Info) const
{
	if(UObject* Object = FTabPayload_WorkspaceDocument::CastChecked<UObject>(Info.Payload.ToSharedRef()))
	{
		return MakeAttributeLambda([WeakObject = TWeakObjectPtr<UObject>(Object)]()
		{
			if(UObject* Object = WeakObject.Get())
			{
				if(Object->GetPackage()->IsDirty())
				{
					return LOCTEXT("TabSuffixAsterisk", "*");
				}
			}

			return FText::GetEmpty();
		});
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FAssetDocumentSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<SWidget> TabContent = SNullWidget::NullWidget;

	const TSharedPtr<FWorkspaceEditor> WorkspaceEditor = HostingAppPtr.Pin();
	check(WorkspaceEditor.IsValid());

	UObject* DocumentID = FTabPayload_WorkspaceDocument::CastChecked<UObject>(Info.Payload.ToSharedRef());
	const FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");

	const FWorkspaceOutlinerItemExport& Export = FTabPayload_WorkspaceDocument::GetExport(Info.Payload.ToSharedRef());
	FWorkspaceEditorContext Context(WorkspaceEditor.ToSharedRef(), DocumentID, Export);
	if(const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, DocumentID}))
	{
		if(DocumentArgs->OnMakeDocumentWidget.IsBound())
		{
			TabContent = DocumentArgs->OnMakeDocumentWidget.Execute(Context);
		}
	}

	return SNew(SWorkspaceTabWrapper, Info.TabInfo, Context)
	[
		TabContent.ToSharedRef()
	];
}

const FSlateBrush* FAssetDocumentSummoner::GetTabIcon(const FWorkflowTabSpawnInfo& Info) const
{
	if(TSharedPtr<FWorkspaceEditor> WorkspaceEditor = HostingAppPtr.Pin())
	{
		UObject* DocumentID = FTabPayload_WorkspaceDocument::CastChecked<UObject>(Info.Payload.ToSharedRef());
		FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
		const FWorkspaceOutlinerItemExport& Export = FTabPayload_WorkspaceDocument::GetExport(Info.Payload.ToSharedRef());
		if(const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, DocumentID}))
		{
			if(DocumentArgs->OnGetTabIcon.IsBound())
			{
				return DocumentArgs->OnGetTabIcon.Execute(FWorkspaceEditorContext(WorkspaceEditor.ToSharedRef(), DocumentID, Export));
			}
		}

		if (UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
		{
			const FAssetData AssetData(DocumentID);
			if(const UAssetDefinition* AssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForAsset(AssetData))
			{
				const FSlateBrush* ThumbnailBrush = AssetDefinition->GetThumbnailBrush(AssetData, AssetData.AssetClassPath.GetAssetName());
				if(ThumbnailBrush == nullptr)
				{
					return FClassIconFinder::FindThumbnailForClass(DocumentID->GetClass(), NAME_None);
				}
				return ThumbnailBrush;
			}
		}
	}
	return nullptr;
}

}

#undef LOCTEXT_NAMESPACE