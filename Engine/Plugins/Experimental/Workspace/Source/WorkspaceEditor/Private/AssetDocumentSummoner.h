// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkspaceTabPayload.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class FWorkflowCentricApplication;
class SGraphEditor;

namespace UE::Workspace
{
	class FWorkspaceEditor;
}

namespace UE::Workspace
{

struct FAssetDocumentSummoner : public FDocumentTabFactory
{
public:
	// Delegate called to save the state of a document
	DECLARE_DELEGATE_OneParam(FOnSaveDocumentState, UObject*);

	FAssetDocumentSummoner(FName InIdentifier, TSharedPtr<FWorkspaceEditor> InHostingApp, bool bInAllowUnsupportedClasses = false);

	void SetAllowedClassPaths(TConstArrayView<FTopLevelAssetPath> InAllowedClassPaths);

private:
	// FWorkflowTabFactory interface
	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;
	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;
	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;
	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;
	virtual bool IsPayloadSupported(TSharedRef<FTabPayload> Payload) const override;
	virtual TAttribute<FText> ConstructTabLabelSuffix(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual const FSlateBrush* GetTabIcon(const FWorkflowTabSpawnInfo& Info) const override;

	virtual bool IsPayloadValid(TSharedRef<FTabPayload> Payload) const override
	{
		if (Payload->PayloadType == FTabPayload_WorkspaceDocument::DocumentPayloadName)
		{
			return Payload->IsValid();
		}
		return false;
	}

	// Creates the label for the tab
	virtual TAttribute<FText> ConstructTabName(const FWorkflowTabSpawnInfo& Info) const override;

	// The hosting app
	TWeakPtr<FWorkspaceEditor> HostingAppPtr;

	// Command list
	TSharedPtr<FUICommandList> CommandList;

	// Allowed object types
	TArray<FTopLevelAssetPath> AllowedClassPaths;

	// Whether or not to allow objects if AllowedClassPaths does not contain their class
	bool bAllowUnsupportedClasses = false;
};

}
