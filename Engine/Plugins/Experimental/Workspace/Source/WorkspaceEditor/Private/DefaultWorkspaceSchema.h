// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceSchema.h"
#include "DefaultWorkspaceSchema.generated.h"

// Workspace schema allowing all asset types
UCLASS()
class UDefaultWorkspaceSchema : public UWorkspaceSchema
{
	GENERATED_BODY()

	// UWorkspaceSchema interface
	virtual FText GetDisplayName() const { return FText::GetEmpty(); }
	virtual TConstArrayView<FTopLevelAssetPath> GetSupportedAssetClassPaths() const override { return TConstArrayView<FTopLevelAssetPath>(); }
	virtual bool SupportsMultipleAssets() const override { return true; }
	virtual bool CanSaveWorkspace() const override { return true; }
};