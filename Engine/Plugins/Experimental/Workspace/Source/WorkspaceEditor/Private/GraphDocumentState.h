// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceDocumentState.h"
#include "Types/SlateVector2.h"
#include "GraphDocumentState.generated.h"

// Base struct used to persist workspace document state
USTRUCT()
struct FGraphDocumentState : public FWorkspaceDocumentState
{
	GENERATED_BODY()

	FGraphDocumentState() = default;

	FGraphDocumentState(const UObject* InObject, const FWorkspaceOutlinerItemExport& InExport, const UE::Slate::FDeprecateVector2DParameter& InViewLocation, float InZoomAmount)
		: FWorkspaceDocumentState(InObject, InExport)
		, ViewLocation(InViewLocation)
		, ZoomAmount(InZoomAmount)
	{}

	UPROPERTY()
	FDeprecateSlateVector2D ViewLocation = FVector2f::Zero();

	UPROPERTY()
	float ZoomAmount = 0.f;
};
