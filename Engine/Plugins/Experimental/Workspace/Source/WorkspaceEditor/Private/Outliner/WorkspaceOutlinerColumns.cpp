// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceOutlinerColumns.h"

#include "ISourceControlModule.h"
#include "ISourceControlState.h"
#include "WorkspaceEditorModule.h"
#include "WorkspaceOutlinerTreeItem.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "ISourceControlProvider.h"
#include "SourceControlHelpers.h"
#include "Widgets/Images/SImage.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "WorkspaceOutlinerColumns"

namespace UE::Workspace
{
	FName WorkspaceOutlinerFileState("File State");
	FName FWorkspaceOutlinerFileStateColumn::GetID()
	{
		return WorkspaceOutlinerFileState;
	}

	SHeaderRow::FColumn::FArguments FWorkspaceOutlinerFileStateColumn::ConstructHeaderRowColumn()
	{
		return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.DirtyBadge"))
				.ToolTipText(LOCTEXT("FileStatusTooltip", "File status of this entry"))
			]
		];	
	}

	const TSharedRef<SWidget> FWorkspaceOutlinerFileStateColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef Item, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
	{
		if (const FWorkspaceOutlinerTreeItem* TreeItem = Item->CastTo<FWorkspaceOutlinerTreeItem>())
		{
			auto GetWeakPackage = [TreeItem]()
			{
				TWeakObjectPtr<const UPackage> WeakPackage = nullptr;
				if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(TreeItem->Export)))
				{
					WeakPackage = SharedFactory->GetPackage(TreeItem->Export);
				}
				
				if (!WeakPackage.IsValid())
				{
					if(const UObject* Object = TreeItem->Export.GetFirstAssetPath().ResolveObject())
					{
						WeakPackage = Object->GetPackage();
					}
				}		

				return WeakPackage;
			};		
			
			return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.HeightOverride(20.0f)		
			[
				SNew(SImage)
				.ToolTipText_Lambda([this, GetWeakPackage]() -> FText
				{
					TWeakObjectPtr<const UPackage> WeakPackage = GetWeakPackage();					
					if (const UPackage* Package = WeakPackage.Get())
					{
						return FText::Format(LOCTEXT("FileStatusTooltipFormat", "File: {0}"), FText::FromString(SourceControlHelpers::PackageFilename(Package)));
					}
				
					return LOCTEXT("FileStatusTooltip", "File status of this entry");
				})
				.Image_Lambda([this, GetWeakPackage]() -> const FSlateBrush*
				{
					TWeakObjectPtr<const UPackage> WeakPackage = GetWeakPackage();
					if (const UPackage* Package = WeakPackage.Get())
					{
						if (Package->IsDirty())
						{
							return FAppStyle::GetBrush("Icons.DirtyBadge");
						}
					}
					
					return nullptr;
				})
			];
		}
		
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE // "WorkspaceOutlinerColumns"