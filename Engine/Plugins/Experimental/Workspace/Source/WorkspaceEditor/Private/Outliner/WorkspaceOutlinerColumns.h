// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "SceneOutlinerFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

class SWidget;

namespace UE::Workspace
{
class FWorkspaceOutlinerFileStateColumn : public ISceneOutlinerColumn
{
public:
	static FName GetID();
	
	FWorkspaceOutlinerFileStateColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}
	virtual ~FWorkspaceOutlinerFileStateColumn() override = default;
	
	// Begin ISceneOutlinerColumn overrides
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }
	// End ISceneOutlinerColumn overrides

private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};
}
