// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceOutlinerHierarchy.h"

#include "ISceneOutlinerMode.h"
#include "Workspace.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "WorkspaceEditorModule.h"
#include "WorkspaceOutlinerTreeItem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"

namespace UE::Workspace
{
	FWorkspaceOutlinerHierarchy::FWorkspaceOutlinerHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorkspace>& InWorkspace) : ISceneOutlinerHierarchy(Mode), WeakWorkspace(InWorkspace)
	{
	}

	void FWorkspaceOutlinerHierarchy::CreateItemsFromAssetData(const FAssetData& AssetData, TArray<FSceneOutlinerTreeItemPtr>& OutItems, TArray<FAssetData>& OutAssets, const FWorkspaceOutlinerItemExport* InParentExport) const
	{
		ProcessedAssetData.AddUnique(AssetData);
		
		FString TagValue;
		if(AssetData.GetTagValue(UE::Workspace::ExportsWorkspaceItemsRegistryTag, TagValue))
		{
			FWorkspaceOutlinerItemExports Exports;
			FWorkspaceOutlinerItemExports::StaticStruct()->ImportText(*TagValue, &Exports, nullptr, 0, nullptr, FWorkspaceOutlinerItemExports::StaticStruct()->GetName());

			const bool bRecursiveReference = OutAssets.Contains(AssetData);

			// Reparented hashes, required to make asset-references (and their child hierarchies) unique inside of the workspace
			TMap<uint32, FWorkspaceOutlinerItemExport> ReparentedHashes;			
			for (const FWorkspaceOutlinerItemExport& Export : Exports.Exports)
			{
				auto AddItem = [this, &OutItems](const FWorkspaceOutlinerItemExport& Export)
				{
					// Only add item types with valid item details
					if (FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(Export)))
					{
						if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FWorkspaceOutlinerTreeItem>(FWorkspaceOutlinerTreeItem::FItemData{Export}))
						{
							OutItems.Add(Item);
						}
					}
				};

				const bool bIsReference = Export.GetData().GetScriptStruct() == FWorkspaceOutlinerAssetReferenceItemData::StaticStruct();

				// Early-out circular/recursive references
				if (bIsReference && bRecursiveReference)
				{
					continue;
				}

				if (bIsReference)
				{
					const FWorkspaceOutlinerAssetReferenceItemData& ReferenceItemData = Export.GetData().Get<FWorkspaceOutlinerAssetReferenceItemData>();

					// Retrieve AssetRegistry data for the referred asset
					FAssetData ReferredAssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(ReferenceItemData.ReferredObjectPath);

					if (ReferredAssetData.IsValid())
					{
						TArray<FSceneOutlinerTreeItemPtr> ReferredItems;
						// Make local copy of AssetData array for recursion checks
						TArray<FAssetData> ReferenceAssets = OutAssets;
						ReferenceAssets.Add(AssetData);
					
						if (InParentExport)
						{
							// Asset reference not at root-level of the workspace, meaning its root entry has to be reparented (in case the find fails, the asset registry exporting is not being done in the same order as outliner representation)
							// Populate export to-be-used by root-level asset referenced export 
							const FWorkspaceOutlinerItemExport ReparentedExport(Export.GetIdentifier(), ReparentedHashes.FindChecked(Export.GetParentHash()), Export.GetData());
							ReparentedHashes.Add(GetTypeHash(Export), ReparentedExport);
							CreateItemsFromAssetData(ReferredAssetData, ReferredItems, ReferenceAssets, &ReparentedExport);
						}
						else
						{
							// Reference in root-level workspace asset
							CreateItemsFromAssetData(ReferredAssetData, ReferredItems,ReferenceAssets,  &Export);
						}

						if(ReferredItems.Num())
						{
							OutItems.Append(ReferredItems);
						}
					}
				
				}

				// Handle exports for referenced assets
				if (InParentExport)
				{
					// Root export for asset
					if (Export.GetParentIdentifier() == NAME_None)
					{
						FWorkspaceOutlinerItemExport ReferenceExport = *InParentExport;
						ensure(ReferenceExport.GetData().GetScriptStruct() == FWorkspaceOutlinerAssetReferenceItemData::StaticStruct());
						
						// Save a copy of the original assets root export on the (to-be-added) asset reference export data
						FWorkspaceOutlinerAssetReferenceItemData& ItemData = ReferenceExport.GetData().GetMutable<FWorkspaceOutlinerAssetReferenceItemData>();
						ItemData.ReferredExport = Export;
						ItemData.bRecursiveReference = bRecursiveReference;

						// Add this new reference export as new root with parent hash of original root asset export
						ReparentedHashes.Add(GetTypeHash(Export), ReferenceExport);

						AddItem(ReferenceExport);
					}
					else
					{
						// Do not add entries other than root export for recursive references
						if (OutAssets.Contains(AssetData))
						{
							continue;
						}

						// in case the find fails, the asset registry exporting is not being done in the same order as outliner representation
						const FWorkspaceOutlinerItemExport ReparentedExport(Export.GetIdentifier(), ReparentedHashes.FindChecked(Export.GetParentHash()), Export.GetData());
						ReparentedHashes.Add(GetTypeHash(Export), ReparentedExport);

						AddItem(ReparentedExport);
					}
				}
				else
				{
					AddItem(Export);
				}
			}
			
			OutAssets.Add(AssetData);
		}
	}

	void FWorkspaceOutlinerHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
	{
		if (const UWorkspace* Workspace = WeakWorkspace.Get())
		{
			TConstArrayView<FAssetData> AssetDataEntries = Workspace->GetAssetDataEntriesView(true);
			ProcessedAssetData.Reset();

			for (const FAssetData& AssetData : AssetDataEntries)
			{
				TArray<FAssetData> Assets;
				CreateItemsFromAssetData(AssetData, OutItems, Assets, nullptr);
			}
		}
	}

	FSceneOutlinerTreeItemPtr FWorkspaceOutlinerHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
	{
		if (const FWorkspaceOutlinerTreeItem* TreeItem = Item.CastTo<FWorkspaceOutlinerTreeItem>())
		{
			const uint32 ParentHash = TreeItem->Export.GetParentHash();
			if (ParentHash != INDEX_NONE)
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentHash))
				{
					return *ParentItem;
				}
				else if(bCreate)
				{
					if (const UWorkspace* Workspace = WeakWorkspace.Get())
					{
						TConstArrayView<FAssetData> AssetDataEntries = Workspace->GetAssetDataEntriesView();
						if(const FAssetData* AssetDataPtr = AssetDataEntries.FindByPredicate([AssetPath = TreeItem->Export.GetTopLevelAssetPath()](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath() == AssetPath; }))
						{
							FString TagValue;
							if(AssetDataPtr->GetTagValue(UE::Workspace::ExportsWorkspaceItemsRegistryTag, TagValue))
							{
								FWorkspaceOutlinerItemExports Exports;
								FWorkspaceOutlinerItemExports::StaticStruct()->ImportText(*TagValue, &Exports, nullptr, 0, nullptr, FWorkspaceOutlinerItemExports::StaticStruct()->GetName());
								
								const FName ParentIdentifier = TreeItem->Export.GetParentIdentifier();
								if (const FWorkspaceOutlinerItemExport* ExportPtr = Exports.Exports.FindByPredicate([ParentIdentifier](const FWorkspaceOutlinerItemExport& ItemExport)
								{
									return ItemExport.GetIdentifier() == ParentIdentifier;
								}))
								{
									Mode->CreateItemFor<FWorkspaceOutlinerTreeItem>(FWorkspaceOutlinerTreeItem::FItemData{*ExportPtr}, true);
								}
							}
						}
					}
				}
			}
		}
		
		return nullptr;
	}
}
