// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"

class UWorkspace;
struct FWorkspaceOutlinerItemExport;

namespace UE::Workspace
{
	class FWorkspaceOutlinerHierarchy : public ISceneOutlinerHierarchy
	{
	public:
		FWorkspaceOutlinerHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorkspace>& InWorkspace);
		FWorkspaceOutlinerHierarchy(const FWorkspaceOutlinerHierarchy&) = delete;
		FWorkspaceOutlinerHierarchy& operator=(const FWorkspaceOutlinerHierarchy&) = delete;

		// Begin ISceneOutlinerHierarchy overrides
		virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
		virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override {}
		virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
		// End ISceneOutlinerHierarchy overrides
	protected:
		void CreateItemsFromAssetData(const FAssetData& AssetData, TArray<FSceneOutlinerTreeItemPtr>& OutItems, TArray<FAssetData>& OutAssets, const FWorkspaceOutlinerItemExport* InParentExport = nullptr) const;
	private:
		friend class SWorkspaceView;
		TWeakObjectPtr<UWorkspace> WeakWorkspace;
		// Set of assets referenced by currently from current workspace,(includes UWorkspace::AssetEntries + any expanded asset references)
		mutable TArray<FAssetData> ProcessedAssetData;
	};
}