// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceOutlinerMode.h"

#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "WorkspaceItemMenuContext.h"
#include "WorkspaceOutlinerHierarchy.h"
#include "WorkspaceOutlinerTreeItem.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Workspace.h"
#include "IWorkspaceEditor.h"
#include "WorkspaceEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "WorkspaceDragDropOperation.h"
#include "WorkspaceEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"
#include "Algo/ForEach.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "FWorkspaceOutlinerMode"

namespace UE::Workspace
{
FWorkspaceOutlinerMode::FWorkspaceOutlinerMode(SSceneOutliner* InSceneOutliner, const TWeakObjectPtr<UWorkspace>& InWeakWorkspace, const TWeakPtr<IWorkspaceEditor>& InWeakWorkspaceEditor) : ISceneOutlinerMode(InSceneOutliner), WeakWorkspace(InWeakWorkspace), WeakWorkspaceEditor(InWeakWorkspaceEditor)
{
	CommandList = MakeShared<FUICommandList>();
	
	if (UWorkspace* Workspace = WeakWorkspace.Get())
	{
		Workspace->ModifiedDelegate.AddRaw(this, &FWorkspaceOutlinerMode::OnWorkspaceModified);
	}

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::LoadModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName))
	{
		AssetRegistryModule->Get().OnAssetUpdated().AddRaw(this, &FWorkspaceOutlinerMode::OnAssetRegistryAssetUpdate);
	}
}

FWorkspaceOutlinerMode::~FWorkspaceOutlinerMode()
{
	if (UWorkspace* Workspace = WeakWorkspace.Get())
	{
		Workspace->ModifiedDelegate.RemoveAll(this);
	}

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::LoadModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName))
	{
		AssetRegistryModule->Get().OnAssetUpdated().RemoveAll(this);
	}
}

void FWorkspaceOutlinerMode::Rebuild()
{
	Hierarchy = CreateHierarchy();
}

TSharedPtr<SWidget> FWorkspaceOutlinerMode::CreateContextMenu()
{
	static const FName MenuName("WorkspaceOutliner.ItemContextMenu");
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);
		if (UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName))
		{
			TWeakPtr<SSceneOutliner> WeakOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
			Menu->AddDynamicSection(TEXT("Assets"), FNewToolMenuDelegate::CreateLambda([WeakOutliner](UToolMenu* InMenu)
			{
				const UAssetEditorToolkitMenuContext* EditorContext = InMenu->FindContext<UAssetEditorToolkitMenuContext>();
				const UWorkspaceItemMenuContext* MenuContext = InMenu->FindContext<UWorkspaceItemMenuContext>();
				if(EditorContext && MenuContext)
				{
					FToolMenuSection& CommonSection = InMenu->AddSection("Common", LOCTEXT("CommonSectionLabel", "Common"));
					CommonSection.AddMenuEntryWithCommandList(FWorkspaceAssetEditorCommands::Get().Open, MenuContext->WeakCommandList.Pin());
					CommonSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Delete, MenuContext->WeakCommandList.Pin(), LOCTEXT("RemoveLabel", "Remove"), LOCTEXT("RemoveTooltip", "Remove current selection"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus"));
					CommonSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Rename, MenuContext->WeakCommandList.Pin());
					// TODO JDB more possible actions? (find in content browser?)

					const bool bSelectionContainsValidAssetPath = MenuContext->SelectedExports.Num() && MenuContext->SelectedExports.ContainsByPredicate([](const FWorkspaceOutlinerItemExport& Export)
					{
						return Export.GetFirstAssetPath().IsValid() || Export.GetResolvedExport().GetFirstAssetPath().IsValid();
					});
					
					FToolMenuSection& AssetsSection = InMenu->AddSection("Assets", LOCTEXT("AssetSectionLabel", "Assets"));
					if (bSelectionContainsValidAssetPath)
					{
						AssetsSection.AddMenuEntry(TEXT("BrowseToAsset"),
							LOCTEXT("BrowseToAssetLabel", "Browse to Asset"),
							LOCTEXT("BrowseToAssetTooltip", "Browse to the selected assets in the content browser"),
							FSlateIcon(FAppStyle::Get().GetStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"),
							FUIAction(FExecuteAction::CreateLambda([SelectedExports=MenuContext->SelectedExports]()
								{
									const IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
									FARFilter Filter;	
									for (const FWorkspaceOutlinerItemExport& ItemExport : SelectedExports)
									{
										Filter.SoftObjectPaths.AddUnique(ItemExport.GetFirstAssetPath());
									}

									TArray<FAssetData> AssetDataList;
									AssetRegistry.GetAssets(Filter, AssetDataList);
									if (AssetDataList.IsEmpty() == false)
									{
										GEditor->SyncBrowserToObjects(AssetDataList);
									}
								}))
						);
					}

					auto IsPackageDirty = [](const UPackage* Package)-> bool
					{										
						return (Package && (Package->IsDirty() || Package->GetExternalPackages().ContainsByPredicate([](const UPackage* Package) { return Package->IsDirty(); })));
					};

					AssetsSection.AddMenuEntry("SaveSelectedAssets",
					FText::FormatOrdered(LOCTEXT("SaveSelectedAssets", "Save {0}|plural(one=Asset,other=Assets)"), MenuContext->SelectedExports.Num()),
					FText::FormatOrdered(LOCTEXT("SaveSelectedAssets_ToolTip", "Save the selected {0}|plural(one=Asset,other=Assets)"), MenuContext->SelectedExports.Num()),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
					FUIAction(
						FExecuteAction::CreateLambda([IsPackageDirty, WeakEditor=EditorContext->Toolkit, SelectedExports=MenuContext->SelectedExports, EditingObjects=EditorContext->GetEditingObjects()]()
						{
							if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(WeakEditor.Pin()))
							{
								TArray<UPackage*> SavablePackages;
								for (const FWorkspaceOutlinerItemExport& ItemExport : SelectedExports)
								{
									if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(ItemExport)))
									{
										UPackage* Package = SharedFactory->GetPackage(ItemExport);
										if (IsPackageDirty(Package))
										{
											SavablePackages.AddUnique(Package);								
										}										
									}
									else
									{
										if (UPackage* Package = FindPackage(nullptr, *ItemExport.GetFirstAssetPath().GetLongPackageName()))
										{
											if (IsPackageDirty(Package))
											{
												SavablePackages.AddUnique(Package);
											}
										}
									}
								}
								
								FEditorFileUtils::PromptForCheckoutAndSave(SavablePackages, false, /*bPromptToSave=*/ false);
							}
						}),
						FCanExecuteAction::CreateLambda([IsPackageDirty, WeakEditor=EditorContext->Toolkit, SelectedExports=MenuContext->SelectedExports, EditingObjects=EditorContext->GetEditingObjects()]()
						{
							if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(WeakEditor.Pin()))
							{
								TArray<UPackage*> SavablePackages;
								for (const FWorkspaceOutlinerItemExport& ItemExport : SelectedExports)
								{
									if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(ItemExport)))
									{									
										const UPackage* Package = SharedFactory->GetPackage(ItemExport);
										if (IsPackageDirty(Package))
										{
											return true;
										}
									}
									else if (ItemExport.GetParentIdentifier() == NAME_None)
									{
										if (const UPackage* Package = FindPackage(nullptr, *ItemExport.GetFirstAssetPath().GetLongPackageName()))
										{
											if (IsPackageDirty(Package))
											{
												return true;
											}
										}
									}
								}
							}

							return false;
						})
					));

				}

				if (TSharedPtr<SSceneOutliner> SharedOutliner = WeakOutliner.Pin())
				{
					SharedOutliner->AddSourceControlMenuOptions(InMenu);
				}
			}));
		}
	}

	{
		UWorkspaceItemMenuContext* MenuContext = NewObject<UWorkspaceItemMenuContext>();
		MenuContext->WeakCommandList = CommandList;

		for (const FSceneOutlinerTreeItemPtr& Item : SceneOutliner->GetSelectedItems())
		{
			if (Item->IsA<FWorkspaceOutlinerTreeItem>())
			{
				MenuContext->SelectedExports.Add(Item->CastTo<FWorkspaceOutlinerTreeItem>()->Export);
				MenuContext->SelectedItems.Add(Item);
			}
		}

		FToolMenuContext Context;
		Context.AddObject(MenuContext);

		WeakWorkspaceEditor.Pin()->InitToolMenuContext(Context);
		
		return UToolMenus::Get()->GenerateWidget(MenuName, Context);
	}
}

void FWorkspaceOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	OpenItems({Item} );

	// Double-clicking automatically expands the item
	SceneOutliner->SetItemExpansion(Item, true);
}

FReply FWorkspaceOutlinerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return ISceneOutlinerMode::OnKeyDown(InKeyEvent);
}

void FWorkspaceOutlinerMode::HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const
{
	if (Selection.Num() > 0)
	{
		if (TSharedPtr<IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
		{
			UWorkspaceItemMenuContext* MenuContext = NewObject<UWorkspaceItemMenuContext>();
			MenuContext->SelectedExports.Reserve(Selection.Num());
			Algo::TransformIf(SceneOutliner->GetSelectedItems(), MenuContext->SelectedExports,
					[](const FSceneOutlinerTreeItemPtr& InItem)
					{
						if (FWorkspaceOutlinerTreeItem* TreeItem = InItem->CastTo<FWorkspaceOutlinerTreeItem>())
						{
							return TreeItem->Export.GetIdentifier() != NAME_None;
						}
					
						return false;
					},
					[](const FSceneOutlinerTreeItemPtr& InItem)
					{
						FWorkspaceOutlinerTreeItem* TreeItem = InItem->CastTo<FWorkspaceOutlinerTreeItem>();
						return TreeItem->Export;
					});

			// Singular item selected
			if (MenuContext->SelectedExports.Num() == 1)
			{
				FWorkspaceOutlinerItemExport& SelectedExport = MenuContext->SelectedExports[0];

				bool bHandled = false;

				// Check whether item details should handle the selection
				if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedDetails = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(SelectedExport)))
				{
					FToolMenuContext Context(MenuContext);
					WeakWorkspaceEditor.Pin()->InitToolMenuContext(Context);
					bHandled = SharedDetails->HandleSelected(Context);
				}
				
				if(!bHandled && SelectedExport.HasData())
				{
					TSharedPtr<FStructOnScope> ExportDataView = MakeShared<FStructOnScope>(SelectedExport.GetData().GetScriptStruct(), SelectedExport.GetData().GetMutableMemory());
					// TODO JDB handle struct selections
					//SharedWorkspaceEditor->SetDetailsStruct(ExportDataView);
				}

				UObject* LoadedAsset = SelectedExport.GetTopLevelAssetPath().ResolveObject();
				if (!bHandled && LoadedAsset)
				{
					SharedWorkspaceEditor->SetDetailsObjects({LoadedAsset});
				}
			}
			else if (MenuContext->SelectedExports.Num() > 1)
			{
				const UScriptStruct* DataType = MenuContext->SelectedExports[0].GetData().GetScriptStruct();
				Algo::ForEach(MenuContext->SelectedExports, [&DataType](const FWorkspaceOutlinerItemExport& Export)
				{
					if (DataType != Export.GetData().GetScriptStruct())
					{
						DataType = nullptr;
					}
				});

				const bool bSingleDataType = DataType != nullptr;
				if (bSingleDataType)
				{
					if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedDetails = FWorkspaceEditorModule::GetOutlinerItemDetails(DataType->GetFName()))
					{
						FToolMenuContext Context(MenuContext);
						WeakWorkspaceEditor.Pin()->InitToolMenuContext(Context);
						SharedDetails->HandleSelected(Context);
					}
				}
			}

			SharedWorkspaceEditor->OnOutlinerSelectionChanged().Broadcast(MenuContext->SelectedExports);
		}
	}
	else
	{
		if (TSharedPtr<IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
		{
			SharedWorkspaceEditor->OnOutlinerSelectionChanged().Broadcast(TConstArrayView<FWorkspaceOutlinerItemExport>());
		}
	}
}

void FWorkspaceOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	if (TSharedPtr<IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		FWorkspaceEditorSelectionScope Scope(SharedWorkspaceEditor);		
		SharedWorkspaceEditor->SetGlobalSelection(SceneOutliner->AsShared(), FOnClearGlobalSelection::CreateRaw(this, &FWorkspaceOutlinerMode::ResetOutlinerSelection));
		HandleItemSelection(Selection);
	}
}

bool FWorkspaceOutlinerMode::CanDeleteItem(const ISceneOutlinerTreeItem& Item) const
{
	if (const FWorkspaceOutlinerTreeItem* TreeItem = Item.CastTo<FWorkspaceOutlinerTreeItem>())
	{
		if(TreeItem->ItemDetails.IsValid())
		{
			if ( TreeItem->ItemDetails->CanDelete(TreeItem->Export))
			{
				return true;
			}
		}
		
		if(TreeItem->Export.GetParentIdentifier() == NAME_None)
		{
			// No details, but allow deletion if we are a root item
			return true;
		}
	}
	return false;
}

bool FWorkspaceOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	if (const FWorkspaceOutlinerTreeItem* TreeItem = Item.CastTo<FWorkspaceOutlinerTreeItem>())
	{
		if(TreeItem->ItemDetails.IsValid())
		{
			return TreeItem->ItemDetails->CanRename(TreeItem->Export);
		}
	}
	return false;
}

void FWorkspaceOutlinerMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	CommandList->MapAction( 
		FWorkspaceAssetEditorCommands::Get().Open, 
		FExecuteAction::CreateRaw(this, &FWorkspaceOutlinerMode::Open)
	);
	
	CommandList->MapAction( 
		FGenericCommands::Get().Delete, 
		FExecuteAction::CreateRaw(this, &FWorkspaceOutlinerMode::Delete),
		FCanExecuteAction::CreateRaw(this, &FWorkspaceOutlinerMode::CanDelete)
	);
	
	CommandList->MapAction( 
		FGenericCommands::Get().Rename, 
		FExecuteAction::CreateRaw(this, &FWorkspaceOutlinerMode::Rename),
		FCanExecuteAction::CreateRaw(this, &FWorkspaceOutlinerMode::CanRename)
	);
}

TSharedPtr<FDragDropOperation> FWorkspaceOutlinerMode::CreateDragDropOperation(const FPointerEvent& MouseEvent,
	const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
 	if(InTreeItems.Num() == 0)
	{
		return nullptr;
	}

	TArray<FSoftObjectPath> AssetPaths;
	for (const FSceneOutlinerTreeItemPtr& Item : InTreeItems)
	{
		if (const FWorkspaceOutlinerTreeItem* WorkspaceItem = Item->CastTo<FWorkspaceOutlinerTreeItem>())
		{
			// Root level entries only (asset) 
			if (WorkspaceItem->Export.GetParentIdentifier() == NAME_None)
			{
				const FSoftObjectPath AssetPath = WorkspaceItem->Export.GetFirstAssetPath();
				if (AssetPath.IsValid())
				{
					AssetPaths.AddUnique(AssetPath);
				}
			}
		}
	}

	if (AssetPaths.Num())
	{
		const IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
		FARFilter Filter;	
		Filter.SoftObjectPaths = MoveTemp(AssetPaths);
		
		TArray<FAssetData> AssetDataList;
		AssetRegistry.GetAssets(Filter, AssetDataList);


		TSharedPtr<FWorkspaceDragDropOp> DragDropOp = MakeShared<FWorkspaceDragDropOp>();
		DragDropOp->Init(AssetDataList);
		DragDropOp->Construct();
		
		return DragDropOp;
	}
 
	return nullptr;
}

TUniquePtr<ISceneOutlinerHierarchy> FWorkspaceOutlinerMode::CreateHierarchy()
{
	return MakeUnique<FWorkspaceOutlinerHierarchy>(this, WeakWorkspace);
}

void FWorkspaceOutlinerMode::OnWorkspaceModified(UWorkspace* InWorkspace) const 
{
	ensure(InWorkspace == WeakWorkspace.Get());
	SceneOutliner->FullRefresh();
}

void FWorkspaceOutlinerMode::ResetOutlinerSelection() const
{
	SceneOutliner->ClearSelection();
}

void FWorkspaceOutlinerMode::OpenItems(TArrayView<const FSceneOutlinerTreeItemPtr> Items) const
{
	for (const FSceneOutlinerTreeItemPtr& Item : Items)
	{
		if (const FWorkspaceOutlinerTreeItem* TreeItem = Item->CastTo<FWorkspaceOutlinerTreeItem>())
		{
			bool bHandled = false;

			if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedDetails = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(TreeItem->Export)))
			{
				UWorkspaceItemMenuContext* MenuContext = NewObject<UWorkspaceItemMenuContext>();
				MenuContext->SelectedExports.Add(TreeItem->Export);
				MenuContext->SelectedItems.Add(Item);

				FToolMenuContext Context(MenuContext);
				WeakWorkspaceEditor.Pin()->InitToolMenuContext(Context);
				
				bHandled = SharedDetails->HandleDoubleClick(Context);
			}
			
			if(!bHandled)
			{
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
					AssetEditorSubsystem->OpenEditorForAsset(TreeItem->Export.GetFirstAssetPath());
				}
				else if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(WeakWorkspaceEditor.Pin()))
				{
					SharedWorkspaceEditor->OpenExports({TreeItem->Export});
				}
			}
		}
	}
}

void FWorkspaceOutlinerMode::Open() const
{
	OpenItems(SceneOutliner->GetSelectedItems());
}

void FWorkspaceOutlinerMode::Delete() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 0)
	{
		return;
	}

	UWorkspace* Workspace = WeakWorkspace.Get();
	if (Workspace == nullptr)
	{
		return;
	}

	// Parse out items into differing types and forward to details
	TMap<TSharedPtr<IWorkspaceOutlinerItemDetails>, TArray<FWorkspaceOutlinerItemExport>> DetailsMap;
	
	FARFilter Filter;
	for (TWeakPtr<ISceneOutlinerTreeItem>& SelectedItem : Selection.SelectedItems)
	{
		FSceneOutlinerTreeItemPtr Item = SelectedItem.Pin();
		if(!Item.IsValid())
		{
			continue;
		}

		const FWorkspaceOutlinerTreeItem* TreeItem = Item->CastTo<FWorkspaceOutlinerTreeItem>();
		if (TreeItem == nullptr)
		{
			continue;
		}

		if(TreeItem->Export.GetParentIdentifier() == NAME_None)
		{
			Filter.SoftObjectPaths.AddUnique(TreeItem->Export.GetFirstAssetPath());
		}
		else
		{
			const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedDetails = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(TreeItem->Export));
			if(!SharedDetails.IsValid())
			{
				continue;
			}

			TArray<FWorkspaceOutlinerItemExport>& ItemsToDelete = DetailsMap.FindOrAdd(SharedDetails);
			ItemsToDelete.Add(TreeItem->Export);
		}
	}

	TArray<FAssetData> AssetDataEntriesToRemove;
	const IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
	AssetRegistry.GetAssets(Filter, AssetDataEntriesToRemove);
	
	if(DetailsMap.Num() > 0 || AssetDataEntriesToRemove.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveItems", "Remove items from workspace"));

		if(AssetDataEntriesToRemove.Num() > 0)
		{
			Workspace->RemoveAssets(AssetDataEntriesToRemove);
		}

		for(const TPair<TSharedPtr<IWorkspaceOutlinerItemDetails>, TArray<FWorkspaceOutlinerItemExport>>& ItemsToDeletePair : DetailsMap)
		{
			ItemsToDeletePair.Key->Delete(ItemsToDeletePair.Value);
		}
	}
}

bool FWorkspaceOutlinerMode::CanDelete() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 0)
	{
		return false;
	}

	for(TWeakPtr<ISceneOutlinerTreeItem> SelectedItem : Selection.SelectedItems)
	{
		FSceneOutlinerTreeItemPtr ItemToDelete = SelectedItem.Pin();
		if(!ItemToDelete.IsValid())
		{
			continue;
		}

		if(!CanDeleteItem(*ItemToDelete) || !ItemToDelete->CanInteract())
		{
			return false;
		}
	}
	return true;
}

void FWorkspaceOutlinerMode::Rename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

		if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
		{
			SceneOutliner->SetPendingRenameItem(ItemToRename);
			SceneOutliner->ScrollItemIntoView(ItemToRename);
		}
	}
}

bool FWorkspaceOutlinerMode::CanRename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
		return ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract();
	}
	return false;
}

void FWorkspaceOutlinerMode::OnAssetRegistryAssetUpdate(const FAssetData& AssetData) const
{
	SceneOutliner->FullRefresh();
}
}	

#undef LOCTEXT_NAMESPACE // "FWorkspaceOutlinerMode"
