// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerMode.h"

class UWorkspace;

namespace UE::Workspace
{
	class IWorkspaceEditor;

	class FWorkspaceOutlinerMode : public ISceneOutlinerMode
	{
	public:
		FWorkspaceOutlinerMode(SSceneOutliner* InSceneOutliner, const TWeakObjectPtr<UWorkspace>& InWeakWorkspace, const TWeakPtr<IWorkspaceEditor>& InWeakWorkspaceEditor);
		virtual ~FWorkspaceOutlinerMode() override;

		// Begin ISceneOutlinerMode overrides
		virtual void Rebuild() override;
		virtual TSharedPtr<SWidget> CreateContextMenu() override;		
		virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
		virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
		virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
		virtual bool CanCustomizeToolbar() const override { return true; }
		virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
		virtual bool CanDelete() const override;
		virtual bool CanRename() const override;
		virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
		virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
		virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	protected:
		virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;		
		// End ISceneOutlinerMode overrides

		void HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const;
		void OnWorkspaceModified(UWorkspace* InWorkspace) const;
		void ResetOutlinerSelection() const;

		void Open() const;
		void Delete() const;
		bool CanDeleteItem(const ISceneOutlinerTreeItem& Item) const;
		void Rename() const;

		void OpenItems(TArrayView<const FSceneOutlinerTreeItemPtr> Items) const;

		void OnAssetRegistryAssetUpdate(const FAssetData& AssetData) const;
	private:
		TWeakObjectPtr<UWorkspace> WeakWorkspace;
		TWeakPtr<IWorkspaceEditor> WeakWorkspaceEditor;
		TSharedPtr<FUICommandList> CommandList;
	};
}