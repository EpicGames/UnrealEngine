// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceOutlinerTreeItem.h"

#include "ISceneOutliner.h"
#include "Styling/SlateColor.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "WorkspaceEditorModule.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "WorkspaceOutlinerTreeItem"

namespace UE::Workspace
{
	const FSceneOutlinerTreeItemType FWorkspaceOutlinerTreeItem::Type;
	class SWorkspaceOutlinerTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SWorkspaceOutlinerTreeLabel) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FWorkspaceOutlinerTreeItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
		{
			WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
			TreeItem = StaticCastSharedRef<FWorkspaceOutlinerTreeItem>(InTreeItem.AsShared());

			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
				[
					SNew(SBox)
					.WidthOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
					.HeightOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
					[
						SNew(SImage)
						.Image_Lambda([this]() -> const FSlateBrush* 
						{
							if (const TSharedPtr<FWorkspaceOutlinerTreeItem> SharedTreeItem = TreeItem.Pin())
							{
								if (SharedTreeItem->ItemDetails.IsValid())
								{
									return SharedTreeItem->ItemDetails->GetItemIcon(SharedTreeItem->Export);
								}
							}
							
							return FAppStyle::GetBrush(TEXT("ClassIcon.Default"));
						})
						.ColorAndOpacity_Lambda([this]() -> FSlateColor
						{
							if (const TSharedPtr<FWorkspaceOutlinerTreeItem> SharedTreeItem = TreeItem.Pin())
							{
								if (SharedTreeItem->ItemDetails.IsValid())
								{
									return SharedTreeItem->ItemDetails->GetItemColor(SharedTreeItem->Export);
								}
							}
							
							return FSlateColor::UseForeground();
						})
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 2.0f)
				[
					SAssignNew(TextBlock, SInlineEditableTextBlock)
					.Text(this, &SWorkspaceOutlinerTreeLabel::GetDisplayText)
					.HighlightText(SceneOutliner.GetFilterHighlightText())
					.ColorAndOpacity(this, &SWorkspaceOutlinerTreeLabel::GetForegroundColor)
					.OnTextCommitted(this, &SWorkspaceOutlinerTreeLabel::OnTextCommited)
					.OnVerifyTextChanged(this, &SWorkspaceOutlinerTreeLabel::OnVerifyTextChanged)
					.ToolTipText(this, &SWorkspaceOutlinerTreeLabel::GetToolTipText)
				]
			];
		}

		FText GetDisplayText() const
		{
			if (const TSharedPtr<FWorkspaceOutlinerTreeItem> Item = TreeItem.Pin())
			{
				return FText::FromString(Item->GetDisplayString());
			}
			return FText();
		}

		FText GetToolTipText() const
		{
			if (const TSharedPtr<FWorkspaceOutlinerTreeItem> Item = TreeItem.Pin())
			{
				return Item->GetToolTipText();
			}
			return FText();
		}

		void OnTextCommited(const FText& InLabel, ETextCommit::Type InCommitInfo) const
		{
			if(InCommitInfo == ETextCommit::OnEnter)
			{
				if (const TSharedPtr<FWorkspaceOutlinerTreeItem> Item = TreeItem.Pin())
				{
					if(Item->ItemDetails.IsValid())
					{
						Item->ItemDetails->Rename(Item->Export, InLabel); 
					}
				}
			}
		}

		bool OnVerifyTextChanged(const FText& InLabel, FText& OutErrorMessage) const
		{
			if (const TSharedPtr<FWorkspaceOutlinerTreeItem> Item = TreeItem.Pin())
			{
				if(Item->ItemDetails.IsValid())
				{
					return Item->ItemDetails->ValidateName(Item->Export, InLabel, OutErrorMessage); 
				}
			}
			return false;
		}

		virtual FSlateColor GetForegroundColor() const override
		{
			const TOptional<FLinearColor> BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem.Pin());
			return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
		}

		TWeakPtr<FWorkspaceOutlinerTreeItem> TreeItem;
		TSharedPtr<SInlineEditableTextBlock> TextBlock;
	};

	FWorkspaceOutlinerTreeItem::FWorkspaceOutlinerTreeItem(const FItemData& InItemData) : ISceneOutlinerTreeItem(FWorkspaceOutlinerTreeItem::Type), Export(InItemData.Export)
	{
		ItemDetails = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(Export));
		if (ItemDetails.IsValid())
		{
			Flags.bIsExpanded = ItemDetails->IsExpandedByDefault();
		}
	}

	bool FWorkspaceOutlinerTreeItem::IsValid() const
	{
		return Export.GetIdentifier().IsValid();
	}

	FSceneOutlinerTreeItemID FWorkspaceOutlinerTreeItem::GetID() const
	{
		return GetTypeHash(Export);
	}

	FString FWorkspaceOutlinerTreeItem::GetDisplayString() const
	{
		if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(Export)))
		{
			return SharedFactory->GetDisplayString(Export);
		}
		
		return Export.GetIdentifier().ToString();
	}

	FText FWorkspaceOutlinerTreeItem::GetToolTipText() const
	{
		if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(Export)))
		{
			return SharedFactory->GetToolTipText(Export);
		}
		
		return FText::GetEmpty();
	}

	TSharedRef<SWidget> FWorkspaceOutlinerTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TSharedRef<SWorkspaceOutlinerTreeLabel> LabelWidget = SNew(SWorkspaceOutlinerTreeLabel, *this, Outliner, InRow);
		RenameRequestEvent.BindSP(LabelWidget->TextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		return LabelWidget;
	}

	FString FWorkspaceOutlinerTreeItem::GetPackageName() const
	{
		if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(MakeOutlinerDetailsId(Export)))
		{
			const UPackage* Package = SharedFactory->GetPackage(Export);
			return Package != nullptr ? Package->GetName() : FString();
		}
		else if (Export.GetParentIdentifier() == NAME_None && Export.GetFirstAssetPath().IsValid())
		{
			return Export.GetFirstAssetPath().GetLongPackageName();
		}	
		
		return ISceneOutlinerTreeItem::GetPackageName();
	}
}

#undef LOCTEXT_NAMESPACE // "WorkspaceOutlinerTreeItem"