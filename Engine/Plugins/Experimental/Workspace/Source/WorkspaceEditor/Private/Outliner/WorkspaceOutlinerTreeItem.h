// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "WorkspaceAssetRegistryInfo.h"

namespace UE::Workspace
{
	class IWorkspaceOutlinerItemDetails;
	struct FWorkspaceOutlinerTreeItem : ISceneOutlinerTreeItem
	{	
		static const FSceneOutlinerTreeItemType Type;
		struct FItemData
		{
			const FWorkspaceOutlinerItemExport& Export;
		};
				
		FWorkspaceOutlinerTreeItem(const FItemData& InItemData);
		virtual ~FWorkspaceOutlinerTreeItem() override = default;

		// Begin ISceneOutlinerTreeItem overrides
		virtual bool IsValid() const override;
		virtual FSceneOutlinerTreeItemID GetID() const override;
		virtual FString GetDisplayString() const override;
		virtual FText GetToolTipText() const;
		virtual bool CanInteract() const override { return true; }
		virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
		virtual FString GetPackageName() const override;
		// End ISceneOutlinerTreeItem overrides

		// AssetRegistry export this tree item represents in the Outliner 
		FWorkspaceOutlinerItemExport Export;
		// Cached _optional_ details provider for the FWorkspaceOutlinerItemExport its inner data
		TSharedPtr<IWorkspaceOutlinerItemDetails> ItemDetails;
	};
}
