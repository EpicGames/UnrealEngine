// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphDocument.h"

#include "Framework/Commands/GenericCommands.h"
#include "WorkspaceEditor.h"
#include "EdGraph/EdGraph.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GraphEditorActions.h"

namespace UE::Workspace
{

void SGraphDocument::Construct(const FArguments& InArgs, TSharedRef<FWorkspaceEditor> InHostingApp, const FWorkspaceDocument& InWorkspaceDocument)
{
	HostingAppPtr = InHostingApp;
	EdGraph = InWorkspaceDocument.GetTypedObjectChecked<UEdGraph>();
	Document = InWorkspaceDocument;
	
	BindCommands();

	OnCanDeleteSelectedNodes = InArgs._OnCanDeleteSelectedNodes;
	OnDeleteSelectedNodes = InArgs._OnDeleteSelectedNodes;
	OnCanCutSelectedNodes = InArgs._OnCanCutSelectedNodes;
	OnCutSelectedNodes = InArgs._OnCutSelectedNodes;
	OnCanCopySelectedNodes = InArgs._OnCanCopySelectedNodes;
	OnCopySelectedNodes = InArgs._OnCopySelectedNodes;
	OnCanPasteNodes = InArgs._OnCanPasteNodes;
	OnPasteNodes = InArgs._OnPasteNodes;
	OnCanDuplicateSelectedNodes = InArgs._OnCanDuplicateSelectedNodes;
	OnDuplicateSelectedNodes = InArgs._OnDuplicateSelectedNodes;
	OnCanOpenInNewTab = InArgs._OnCanOpenInNewTab;
	OnOpenInNewTab = InArgs._OnOpenInNewTab;
	OnCreateComment = InArgs._OnCreateComment;

	SGraphEditor::FGraphEditorEvents Events;
	Events.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateLambda([this, OnCreateActionMenu = InArgs._OnCreateActionMenu](UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bInAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
	{
		if(OnCreateActionMenu.IsBound())
		{
			return OnCreateActionMenu.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), InGraph, FDeprecateSlateVector2D(InNodePosition), InDraggedPins, bInAutoExpand, InOnMenuClosed);
		}
		return FActionMenuContent();
	});
	Events.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateLambda([this, OnGraphSelectionChanged = InArgs._OnGraphSelectionChanged](const TSet<UObject*>& NewSelection)
	{
		if (const TSharedPtr<FWorkspaceEditor> SharedWorkspaceEditor = HostingAppPtr.Pin())
		{
			SharedWorkspaceEditor->SetGlobalSelection(AsShared(), FOnClearGlobalSelection::CreateRaw(this, &SGraphDocument::OnResetSelection));
		}
			
		OnGraphSelectionChanged.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), NewSelection);
			
	});
	Events.OnTextCommitted = ::FOnNodeTextCommitted::CreateLambda([this, OnNodeTextCommitted = InArgs._OnNodeTextCommitted](const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
	{
		OnNodeTextCommitted.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), NewText, CommitInfo, NodeBeingChanged);
	});
	Events.OnNodeDoubleClicked = FSingleNodeEvent::CreateLambda([this, OnNodeDoubleClicked = InArgs._OnNodeDoubleClicked](UEdGraphNode* InNode)
	{
		OnNodeDoubleClicked.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), InNode);
	});

	ChildSlot
	[
		SAssignNew(GraphEditor, SGraphEditor)
		.AdditionalCommands(CommandList)
		.IsEditable(this, &SGraphDocument::IsEditable, EdGraph)
		.GraphToEdit(EdGraph)
		.GraphEvents(Events)
		.AssetEditorToolkit(HostingAppPtr)
		.OnNavigateHistoryBack(InArgs._OnNavigateHistoryBack)		
		.OnNavigateHistoryForward(InArgs._OnNavigateHistoryForward)
	];
}

void SGraphDocument::BindCommands()
{
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SGraphDocument::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanDeleteSelectedNodes));

	CommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SGraphDocument::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanCutSelectedNodes));

	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SGraphDocument::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanCopySelectedNodes));

	CommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SGraphDocument::PasteNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanPasteNodes));

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SGraphDocument::DuplicateSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanDuplicateSelectedNodes));

	CommandList->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateSP(this, &SGraphDocument::SelectAllNodes),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanSelectAllNodes));

	CommandList->MapAction(FGraphEditorCommands::Get().OpenInNewTab,
		FExecuteAction::CreateSP(this, &SGraphDocument::OpenInNewTab),
		FCanExecuteAction::CreateSP(this, &SGraphDocument::CanOpenInNewTab));

	CommandList->MapAction(FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &SGraphDocument::CreateComment));
		
	TWeakPtr<SGraphDocument> WeakDocument = StaticCastSharedRef<SGraphDocument>(AsShared());
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesTop, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignTop(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesTop, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignTop(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignMiddle(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesBottom, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignBottom(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesLeft, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignLeft(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesCenter, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignCenter(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().AlignNodesRight, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnAlignRight(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().StraightenConnections, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnStraightenConnections(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnDistributeNodesH(); }}));
	CommandList->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically, FExecuteAction::CreateSPLambda(this, [WeakDocument](){ if (TSharedPtr<SGraphDocument> SharedDocument = WeakDocument.Pin()) { SharedDocument->GraphEditor->OnDistributeNodesV(); }}));
}

bool SGraphDocument::CanDeleteSelectedNodes() const
{
	const bool bCan = OnCanDeleteSelectedNodes.IsBound()
		? IsEditable(EdGraph) && OnCanDeleteSelectedNodes.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetSelectedNodes())
		: false;
	return bCan;
}

void SGraphDocument::DeleteSelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	OnDeleteSelectedNodes.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), SelectedNodes);
}

bool SGraphDocument::CanCutSelectedNodes() const
{
	const bool bCan = OnCanCutSelectedNodes.IsBound() 
		? IsEditable(EdGraph) && OnCanCutSelectedNodes.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetSelectedNodes())
		: false;
	return bCan;
}

void SGraphDocument::CutSelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	OnCutSelectedNodes.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), SelectedNodes);
}

bool SGraphDocument::CanCopySelectedNodes() const
{
	const bool bCan = OnCanCopySelectedNodes.IsBound()
		? OnCanCopySelectedNodes.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetSelectedNodes())
		: false;
	return bCan;
}

void SGraphDocument::CopySelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	OnCopySelectedNodes.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), SelectedNodes);
}

bool SGraphDocument::CanPasteNodes() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	const bool bCan = OnCanPasteNodes.IsBound()
		? IsEditable(EdGraph) && !TextToImport.IsEmpty() && OnCanPasteNodes.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), TextToImport)
		: false;
	return bCan;
}

void SGraphDocument::PasteNodes()
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	OnPasteNodes.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetPasteLocation2f(), TextToImport);
}

bool SGraphDocument::CanDuplicateSelectedNodes() const
{
	const bool bCan = OnCanDuplicateSelectedNodes.IsBound()
		? IsEditable(EdGraph) && OnCanDuplicateSelectedNodes.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetSelectedNodes())
		: false;
	return bCan;
}

void SGraphDocument::DuplicateSelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	OnDuplicateSelectedNodes.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetPasteLocation2f(), SelectedNodes);
}

bool SGraphDocument::CanSelectAllNodes() const
{
	const bool bCan = GraphEditor.IsValid();
	return bCan;
}

void SGraphDocument::SelectAllNodes()
{
	GraphEditor->SelectAllNodes();
}

bool SGraphDocument::CanOpenInNewTab() const
{
	const bool bCan = OnCanOpenInNewTab.IsBound()
		? OnCanOpenInNewTab.Execute(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetSelectedNodes())
		: false;
	return bCan;
}

void SGraphDocument::OpenInNewTab()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	OnOpenInNewTab.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), SelectedNodes);
}

void SGraphDocument::CreateComment()
{
	OnCreateComment.ExecuteIfBound(FWorkspaceEditorContext(HostingAppPtr.Pin().ToSharedRef(), Document), GraphEditor->GetPasteLocation2f());
}

bool SGraphDocument::IsEditable(UEdGraph* InGraph) const
{
	return InGraph && HostingAppPtr.Pin()->InEditingMode() && InGraph->bEditable;
}

void SGraphDocument::OnResetSelection()
{
	GraphEditor->ClearSelectionSet();
}
};
