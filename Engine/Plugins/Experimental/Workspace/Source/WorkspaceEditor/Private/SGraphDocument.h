// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditor.h"
#include "IWorkspaceEditorModule.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Workspace
{
	class FWorkspaceEditor;
	class FWorkspaceEditorModule;
}

namespace UE::Workspace
{

// Wrapper widget for a graph editor
class SGraphDocument : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SGraphDocument) {}

	SLATE_EVENT(FOnCreateActionMenu, OnCreateActionMenu)

	SLATE_EVENT(FOnNodeTextCommitted, OnNodeTextCommitted)

	SLATE_EVENT(FOnGraphSelectionChanged, OnGraphSelectionChanged)

	SLATE_EVENT(FOnCanPerformActionOnSelectedNodes, OnCanDeleteSelectedNodes)
	SLATE_EVENT(FOnPerformActionOnSelectedNodes, OnDeleteSelectedNodes)

	SLATE_EVENT(FOnCanPerformActionOnSelectedNodes, OnCanCutSelectedNodes)
	SLATE_EVENT(FOnPerformActionOnSelectedNodes, OnCutSelectedNodes)

	SLATE_EVENT(FOnCanPerformActionOnSelectedNodes, OnCanCopySelectedNodes)
	SLATE_EVENT(FOnPerformActionOnSelectedNodes, OnCopySelectedNodes)

	SLATE_EVENT(FOnCanPasteNodes, OnCanPasteNodes)
	SLATE_EVENT(FOnPasteNodes, OnPasteNodes)

	SLATE_EVENT(FOnCanPerformActionOnSelectedNodes, OnCanDuplicateSelectedNodes)
	SLATE_EVENT(FOnDuplicateSelectedNodes, OnDuplicateSelectedNodes)

	SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryBack)
	SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryForward)

	SLATE_EVENT(FOnNodeDoubleClicked, OnNodeDoubleClicked)

	SLATE_EVENT(FOnCanPerformActionOnSelectedNodes, OnCanOpenInNewTab)
	SLATE_EVENT(FOnPerformActionOnSelectedNodes, OnOpenInNewTab)

	SLATE_EVENT(FOnCreateComment, OnCreateComment)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FWorkspaceEditor> InHostingApp, const FWorkspaceDocument& InDocument);
	
	void BindCommands();
	

	bool CanDeleteSelectedNodes() const;
	void DeleteSelectedNodes();

	bool CanCutSelectedNodes() const;
	void CutSelectedNodes();

	bool CanCopySelectedNodes() const;
	void CopySelectedNodes();

	bool CanPasteNodes() const;
	void PasteNodes();

	bool CanDuplicateSelectedNodes() const;
	void DuplicateSelectedNodes();

	bool CanSelectAllNodes() const;
	void SelectAllNodes();

	bool CanOpenInNewTab() const;
	void OpenInNewTab();

	void CreateComment();

	bool IsEditable(UEdGraph* InGraph) const;
	void OnResetSelection();

	// The graph we are editing
	UEdGraph* EdGraph = nullptr;
	FWorkspaceDocument Document;

	// The graph editor we wrap
	TSharedPtr<SGraphEditor> GraphEditor;

	// Command list for graphs
	TSharedPtr<FUICommandList> CommandList;

	// The hosting app
	TWeakPtr<FWorkspaceEditor> HostingAppPtr;

	// Delegate called when we delete nodes
	FOnCanPerformActionOnSelectedNodes OnCanDeleteSelectedNodes;
	FOnPerformActionOnSelectedNodes OnDeleteSelectedNodes;

	// Cut delegates
	FOnCanPerformActionOnSelectedNodes OnCanCutSelectedNodes;
	FOnPerformActionOnSelectedNodes OnCutSelectedNodes;

	// Copy delegates
	FOnCanPerformActionOnSelectedNodes OnCanCopySelectedNodes;
	FOnPerformActionOnSelectedNodes OnCopySelectedNodes;

	// Paste delegates
	FOnCanPasteNodes OnCanPasteNodes;
	FOnPasteNodes OnPasteNodes;

	// Duplicate delegates
	FOnCanPerformActionOnSelectedNodes OnCanDuplicateSelectedNodes;
	FOnDuplicateSelectedNodes OnDuplicateSelectedNodes;

	// Open in new Tab delegates
	FOnCanPerformActionOnSelectedNodes OnCanOpenInNewTab;
	FOnPerformActionOnSelectedNodes OnOpenInNewTab;

	FOnCreateComment OnCreateComment;

	friend class FWorkspaceEditorModule;
};

}
