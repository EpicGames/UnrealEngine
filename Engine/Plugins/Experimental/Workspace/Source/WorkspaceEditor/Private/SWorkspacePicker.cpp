// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorkspacePicker.h"

#include "AssetToolsModule.h"
#include "Workspace.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SPrimaryButton.h"
#include "WorkspaceFactory.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SWorkspacePicker"

namespace UE::Workspace
{

void SWorkspacePicker::Construct(const FArguments& InArgs)
{
	WorkspaceAssets = InArgs._WorkspaceAssets;
	WorkspaceFactoryClass = InArgs._WorkspaceFactoryClass;
	HintText = InArgs._HintText;
	
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this, OnWorkspaceSelected = InArgs._OnExistingWorkspaceSelected](const FAssetData& InAssetData)
	{
		SelectedWorkspace = CastChecked<UWorkspace>(InAssetData.GetAsset());
		OnWorkspaceSelected.ExecuteIfBound(SelectedWorkspace);

		if(TSharedPtr<SWindow> Window = WeakWindow.Pin())
		{
			Window->RequestDestroyWindow();
		}
	});
	AssetPickerConfig.Filter.ClassPaths.Add(UWorkspace::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([this](const FAssetData& InAssetData)
	{
		return !WorkspaceAssets.IsEmpty() && !WorkspaceAssets.Contains(InAssetData);
	});

	const FText DisplayHintText = [this]()
	{
		switch (HintText)
		{
			case EHintText::SelectedAssetIsPartOfMultipleWorkspaces:
				return LOCTEXT("SelectedAssetIsPartOfMultipleWorkspaces", "Selected asset is part of multiple workspaces.\nPlease select the workspace you want to open the asset with");
			case EHintText::CreateOrUseExistingWorkspace:
				return LOCTEXT("CreateOrUseExistingWorkspace", "Create or select an existing workspace to associate assets with.");
			default:
				return FText();
		}
	}();

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(400.0f)
		.HeightOverride(400.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(DisplayHintText)
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(5.0f)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			.HAlign(HAlign_Fill)
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("CreateNewWorkspace", "Create New Workspace"))
				.OnClicked_Lambda([this, OnWorkspaceSelected = InArgs._OnNewWorkspaceCreated]()
				{
					UWorkspaceFactory* Factory = NewObject<UWorkspaceFactory>(GetTransientPackage(), WorkspaceFactoryClass.Get());
					UPackage* Package = CreatePackage(nullptr);
					FName PackageName = *FPaths::GetBaseFilename(Package->GetName());
					if (Factory->ConfigureProperties())
					{
						FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
						UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(UWorkspace::StaticClass(), Factory);

						if (NewAsset)
						{
							SelectedWorkspace = CastChecked<UWorkspace>(NewAsset);
							OnWorkspaceSelected.ExecuteIfBound(SelectedWorkspace);
						}
					}
					
					if(TSharedPtr<SWindow> Window = WeakWindow.Pin())
					{
						Window->RequestDestroyWindow();
					}
					
					return FReply::Handled();
				})
			]
		]
	];
}

void SWorkspacePicker::ShowModal()
{
	TSharedRef<SWindow> Window = SNew(SWindow)
	.Title(LOCTEXT("ChooseWorkspaceToOpen", "Choose Workspace to Open"))
	.SizingRule(ESizingRule::Autosized)
	.SupportsMaximize(false)
	.SupportsMinimize(false)
	[
		AsShared()
	];

	WeakWindow = Window;

	FSlateApplication::Get().AddModalWindow(Window, FGlobalTabmanager::Get()->GetRootWindow());
}

TObjectPtr<UObject> SWorkspacePicker::GetSelectedWorkspace() const
{
	return SelectedWorkspace;
}
	
}

#undef LOCTEXT_NAMESPACE