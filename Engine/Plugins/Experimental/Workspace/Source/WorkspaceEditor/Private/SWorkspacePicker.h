// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "IWorkspacePicker.h"
#include "Widgets/SCompoundWidget.h"

class UWorkspace;
class UWorkspaceFactory;

namespace UE::Workspace
{

class SWorkspacePicker : public SCompoundWidget, public IWorkspacePicker
{
	DECLARE_DELEGATE_OneParam(FOnWorkspaceSelected, TObjectPtr<UWorkspace>)
	
	SLATE_BEGIN_ARGS(SWorkspacePicker) {}

	SLATE_EVENT(FOnWorkspaceSelected, OnExistingWorkspaceSelected)

	SLATE_EVENT(FOnWorkspaceSelected, OnNewWorkspaceCreated)

	// The set of workspace assets the user can choose from
	SLATE_ARGUMENT(TArray<FAssetData>, WorkspaceAssets)

	SLATE_ARGUMENT(IWorkspacePicker::EHintText, HintText)
		
	SLATE_ARGUMENT(TSubclassOf<UWorkspaceFactory>, WorkspaceFactoryClass)

	SLATE_ARGUMENT(bool, bShowSaveDialog)
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// IWorkspacePicker
	virtual void ShowModal() override;
	virtual TObjectPtr<UObject> GetSelectedWorkspace() const override;

private:
	TArray<FAssetData> WorkspaceAssets;

	EHintText HintText = EHintText::SelectedAssetIsPartOfMultipleWorkspaces;

	TSubclassOf<UWorkspaceFactory> WorkspaceFactoryClass;

	TObjectPtr<UWorkspace> SelectedWorkspace;

	TWeakPtr<SWindow> WeakWindow;
};

};