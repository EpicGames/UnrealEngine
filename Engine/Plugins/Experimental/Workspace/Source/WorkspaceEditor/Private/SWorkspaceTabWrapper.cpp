// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorkspaceTabWrapper.h"

#include "EditorModeManager.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "WorkspaceSchema.h"

#define LOCTEXT_NAMESPACE "SWorkspaceTabWrapper"

void SWorkspaceTabWrapper::Construct( const FArguments& InArgs, TSharedPtr<class FTabInfo> InTabInfo, const UE::Workspace::FWorkspaceEditorContext& InEditorContext)
{
	Content = InArgs._Content.Widget;
	WeakWorkspaceEditor = StaticCastSharedRef<UE::Workspace::FWorkspaceEditor>(InEditorContext.WorkspaceEditor);
	WeakDocumentObject = InEditorContext.Document.GetObject();
	Export = InEditorContext.Document.Export;

	if (InTabInfo->GetTab().IsValid())
	{
		InTabInfo->GetTab().Pin()->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([WeakEditor=WeakWorkspaceEditor, WeakObject=WeakDocumentObject](TSharedRef<SDockTab>)
		{
			if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakEditor.Pin())
			{
				SharedWorkspaceEditor->RemoveEditingObject(WeakObject.Get());
			}
		}));
	}

	// Set-up shared breadcrumb defaults JDB TODO figure out correct padding to align fake title with breadcrumbs
    const FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
    const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");

	UToolMenus* ToolMenus = UToolMenus::Get();

	const FName ToolbarName = "WorkspaceTabWrapperToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* ToolBarMenu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolBarMenu->StyleName = "AssetEditorToolbar";
		
		auto Handler = FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UWorkspaceTabMenuContext* Context = InMenu->FindContext<UWorkspaceTabMenuContext>())
			{
				const TSharedRef<SWorkspaceTabWrapper> This = Context->TabWrapper.Pin().ToSharedRef();
				const FToolMenuEntry SaveEntry = FToolMenuEntry::InitToolBarButton(
					"Save",
					FUIAction(
						FExecuteAction::CreateSP(This, &SWorkspaceTabWrapper::ExecuteSave),
						FCanExecuteAction::CreateSP(This, &SWorkspaceTabWrapper::CanExecuteSave),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateSP(This, &SWorkspaceTabWrapper::IsSaveButtonVisible)),
					FText::GetEmpty(),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"));

				const FToolMenuEntry FindInContentBrowserEntry = FToolMenuEntry::InitToolBarButton(
					"FindInContentBrowser",
					FUIAction(FExecuteAction::CreateSP(This, &SWorkspaceTabWrapper::FindObjectInContentBrowser)),
					FText::GetEmpty(),
					LOCTEXT("FindInContentBrowserTooltip", "Finds this asset in the content browser"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "SystemWideCommands.FindInContentBrowser"));

				FToolMenuSection& Section = InMenu->AddSection("AssetSection");
				Section.AddEntry(SaveEntry);
				Section.AddEntry(FindInContentBrowserEntry);
			}
		});

		ToolBarMenu->AddDynamicSection("AssetActions", Handler);
	}
	
	UWorkspaceTabMenuContext* WorkspaceTabContext = NewObject<UWorkspaceTabMenuContext>();
	WorkspaceTabContext->TabWrapper = SharedThis(this);

	const FToolMenuContext MenuContext(WorkspaceTabContext);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InTabInfo->CreateHistoryNavigationWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			// Title text/icon
			+SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 10.0f,5.0f )
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D{20.f})
						.Image( this, &SWorkspaceTabWrapper::GetTabIcon )
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
						.Orientation(Orient_Horizontal)
						.ScrollBarVisibility(EVisibility::Collapsed)

						+SScrollBox::Slot()
						.Padding(0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)
							// show fake 'root' breadcrumb for the title
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(BreadcrumbTrailPadding)
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.FillHeight(1.f)
								[
									SNew(STextBlock)
									.Text(this, &SWorkspaceTabWrapper::GetWorkspaceName)
									.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
									.Visibility( this, &SWorkspaceTabWrapper::IsWorkspaceNameVisible )
								]
								
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(BreadcrumbTrailPadding)
							[
								SNew(SImage)
								.Image( BreadcrumbButtonImage )
								.Visibility( this, &SWorkspaceTabWrapper::IsWorkspaceNameVisible )
								.ColorAndOpacity(FSlateColor::UseForeground())
							]

							// New style breadcrumb
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>>)
								.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
								.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
								.ButtonContentPadding(BreadcrumbTrailPadding)
								.DelimiterImage(BreadcrumbButtonImage)
								.OnCrumbClicked_Lambda([](const TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> InBreadcrumb){ InBreadcrumb->OnClicked.ExecuteIfBound(); })
								.GetCrumbButtonContent_Lambda([](const TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> InBreadcrumb, const FTextBlockStyle* InTextStyle) -> TSharedRef<SWidget>
								{
									FText Text = InBreadcrumb->OnGetLabel.Execute().Get();
									return SNew(SHorizontalBox)
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(STextBlock)
											.Text(Text)
											.TextStyle(InTextStyle)
										]
										+SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(3.0f, 0.f, 0.f, 0.f)
										[
											SNew(SBox)
										   .HAlign(HAlign_Center)
										   .VAlign(VAlign_Center)
										   .HeightOverride(20.0f)		
										   [
											   SNew(SImage)
											   .Image_Lambda([InBreadcrumb]() -> const FSlateBrush*
											   {											
												   if (InBreadcrumb->CanSave.IsBound() && InBreadcrumb->CanSave.Execute())
												   {
													   return FAppStyle::GetBrush("Icons.DirtyBadge");
												   }
										   		
												   return nullptr;
											   })
										   ]
										];
								})
							]
						]
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				UToolMenus::Get()->GenerateWidget(ToolbarName, MenuContext)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			GetContent()
		]
	];

	RebuildBreadcrumbTrail();
} 

void SWorkspaceTabWrapper::RebuildBreadcrumbTrail() const
{
	BreadcrumbTrail->ClearCrumbs(false);

	TArray<TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>> Breadcrumbs;

	if (Export.GetFirstAssetPath().IsValid())
	{
		TArray<FWorkspaceOutlinerItemExport> Exports;
		Export.GetExports(Exports);

		for (const FWorkspaceOutlinerItemExport& DocumentExport : Exports)
		{
			UObject* DocumentID = DocumentExport.GetFirstObjectOfType<UObject>();
			check(DocumentID);

			// [TODO] DocumentID can fail to resolve for previously moved assets (need to resolve redirector?)
			if (DocumentID)
			{
				if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
				{
					const UE::Workspace::FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<UE::Workspace::FWorkspaceEditorModule>("WorkspaceEditor");
					if(const UE::Workspace::FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({DocumentExport, DocumentID}))
					{
						if(DocumentArgs->OnGetDocumentBreadcrumbTrail.IsBound())
						{
							DocumentArgs->OnGetDocumentBreadcrumbTrail.Execute(UE::Workspace::FWorkspaceEditorContext(SharedWorkspaceEditor.ToSharedRef(), { DocumentExport, DocumentID}), Breadcrumbs);
						}
					}
				}
			}
		}
	}
	else
	{
		if (UObject* DocumentID = WeakDocumentObject.Get())
	    {
		    if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
		    {
			    const UE::Workspace::FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<UE::Workspace::FWorkspaceEditorModule>("WorkspaceEditor");
			    if(const UE::Workspace::FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, DocumentID}))
			    {
				    if(DocumentArgs->OnGetDocumentBreadcrumbTrail.IsBound())
				    {
					    DocumentArgs->OnGetDocumentBreadcrumbTrail.Execute(UE::Workspace::FWorkspaceEditorContext(SharedWorkspaceEditor.ToSharedRef(), { FWorkspaceOutlinerItemExport(), DocumentID}), Breadcrumbs);
				    }
			    }
		    }
	    }
	}	

	// Widgets have to be added in reverse order
	for (int32 Index = Breadcrumbs.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> Breadcrumb = Breadcrumbs[Index];
		BreadcrumbTrail->PushCrumb(	TAttribute<FText>::CreateLambda([Breadcrumb]() { return Breadcrumb->OnGetLabel.IsBound() ? Breadcrumb->OnGetLabel.Execute().Get() : FText::GetEmpty(); }), Breadcrumb);
	}
}

const FSlateBrush* SWorkspaceTabWrapper::GetTabIcon() const
{
	if (UObject* DocumentID = WeakDocumentObject.Get())
	{
		if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
		{
			const UE::Workspace::FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<UE::Workspace::FWorkspaceEditorModule>("WorkspaceEditor");
			if(const UE::Workspace::FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, DocumentID}))
			{
				if(DocumentArgs->OnGetTabIcon.IsBound())
				{
					return DocumentArgs->OnGetTabIcon.Execute(UE::Workspace::FWorkspaceEditorContext(SharedWorkspaceEditor.ToSharedRef(), { Export, DocumentID }));
				}
			}
		}			
	}
	
	return nullptr;
}

EVisibility SWorkspaceTabWrapper::IsWorkspaceNameVisible() const
{
	if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		if (SharedWorkspaceEditor->Workspace && SharedWorkspaceEditor->GetSchema()->CanSaveWorkspace())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FText  SWorkspaceTabWrapper::GetWorkspaceName() const
{
	if (const TSharedPtr<UE::Workspace::FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		if (SharedWorkspaceEditor->Workspace)
		{
			return FText::FromName(SharedWorkspaceEditor->Workspace->GetFName());
		}
	}

	return FText::GetEmpty();
}

void SWorkspaceTabWrapper::ExecuteSave() const
{
	if(BreadcrumbTrail.IsValid() && BreadcrumbTrail->HasCrumbs())
	{
		TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> LastBreadcrumb = BreadcrumbTrail->PeekCrumb();
		 LastBreadcrumb->OnSave.ExecuteIfBound();
	}
}

bool SWorkspaceTabWrapper::CanExecuteSave() const
{
	if(BreadcrumbTrail.IsValid() && BreadcrumbTrail->HasCrumbs())
	{
		TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> LastBreadcrumb = BreadcrumbTrail->PeekCrumb();
		return LastBreadcrumb->CanSave.IsBound() ? LastBreadcrumb->CanSave.Execute() : false;
	}

	return false;
}

bool SWorkspaceTabWrapper::IsSaveButtonVisible() const
{
	if(BreadcrumbTrail.IsValid() && BreadcrumbTrail->HasCrumbs())
	{
		TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb> LastBreadcrumb = BreadcrumbTrail->PeekCrumb();
		return LastBreadcrumb->CanSave.IsBound();
	}

	return false;
}

void SWorkspaceTabWrapper::FindObjectInContentBrowser() const
{
	if (WeakDocumentObject.IsValid() && GEditor)
	{
		GEditor->SyncBrowserToObject(WeakDocumentObject.Get());
	}
}

#undef LOCTEXT_NAMESPACE // "SWorkspaceTabWrapper"
