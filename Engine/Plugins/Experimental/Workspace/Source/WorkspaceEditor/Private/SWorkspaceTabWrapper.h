// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Workspace.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkspaceEditor.h"
#include "WorkspaceEditorModule.h"

#include "SWorkspaceTabWrapper.generated.h"

UCLASS()
class UWorkspaceTabMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SWorkspaceTabWrapper> TabWrapper;
};

class SWorkspaceTabWrapper : public SCompoundWidget
{
public:
	SWorkspaceTabWrapper() = default;

	SLATE_BEGIN_ARGS( SWorkspaceTabWrapper ){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedPtr<class FTabInfo> InTabInfo, const UE::Workspace::FWorkspaceEditorContext& InEditorContext);
	TSharedRef<SWidget> GetContent() const { return Content.ToSharedRef(); }
	TWeakObjectPtr<UObject> GetDocumentObject() const { return WeakDocumentObject; }
	UE::Workspace::FWorkspaceDocument GetDocument() const { return { Export, WeakDocumentObject.Get() }; }
	
protected:
	void RebuildBreadcrumbTrail() const;
	const FSlateBrush* GetTabIcon() const;
	EVisibility IsWorkspaceNameVisible() const;
	FText GetWorkspaceName() const;

	void ExecuteSave() const;
	bool CanExecuteSave() const;
	bool IsSaveButtonVisible() const;
	void FindObjectInContentBrowser() const;

	TSharedPtr<SWidget> Content = nullptr;
	TWeakPtr<UE::Workspace::FWorkspaceEditor> WeakWorkspaceEditor = nullptr;
	TWeakObjectPtr<UObject> WeakDocumentObject = nullptr;
	FWorkspaceOutlinerItemExport Export;

	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox = nullptr;
	TSharedPtr<SBreadcrumbTrail<TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>>> BreadcrumbTrail = nullptr;	
};
