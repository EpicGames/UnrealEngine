// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorkspaceView.h"
#include "Workspace.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetDropTarget.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerSourceControlColumn.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "SSceneOutliner.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "WorkspaceSchema.h"
#include "Outliner/WorkspaceOutlinerColumns.h"
#include "Outliner/WorkspaceOutlinerHierarchy.h"
#include "Outliner/WorkspaceOutlinerMode.h"
#include "Outliner/WorkspaceOutlinerTreeItem.h"

#define LOCTEXT_NAMESPACE "SWorkspaceView"

namespace UE::Workspace
{

class SWorkspaceOutliner : public SSceneOutliner
{
public:
	SLATE_BEGIN_ARGS(SWorkspaceOutliner) {}
		SLATE_ATTRIBUTE(bool, CanSaveAll)
		SLATE_EVENT(FOnClicked, OnSaveAll)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions, UWorkspace* InWorkspace)
	{
		AddAssetButton = SNew(SPositiveActionButton)
				.OnGetMenuContent(this, &SWorkspaceOutliner::GetAddAssetPicker)			 
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("AddAssetButton", "Add"));
				
		SaveAllButton = SNew(SActionButton)
			.IsEnabled(InArgs._CanSaveAll)
			.OnClicked(InArgs._OnSaveAll)
			.Text(LOCTEXT("SaveAll", "Save All"))
			.Icon(FAppStyle::GetBrush("MainFrame.SaveAll"));

		WeakWorkspace = InWorkspace;
		
		SSceneOutliner::FArguments OutlinerArgs;
		
		SSceneOutliner::Construct(OutlinerArgs, InitOptions);
	}

	SWorkspaceOutliner() = default;
	virtual ~SWorkspaceOutliner() override = default;

	virtual void CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar) override
	{
		if (UWorkspace* Workspace = WeakWorkspace.Get())
		{
			if (Workspace->GetSchema()->SupportsMultipleAssets())
			{
				Toolbar->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					AddAssetButton.ToSharedRef()
				];
			}
		}
		
		Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SaveAllButton.ToSharedRef()
		];
	}

	TSharedRef<SWidget> GetAddAssetPicker() const
	{
		if (UWorkspace* Workspace = WeakWorkspace.Get())
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			FAssetPickerConfig AssetPickerConfig;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this, Workspace](const FAssetData& InAssetData)
			{
				FScopedTransaction Transaction(LOCTEXT("AddAsset", "Add asset to workspace"));
				Workspace->AddAsset(InAssetData);
		
				AddAssetButton->SetIsMenuOpen(false, false);
			});

			TArray<FAssetData> AssetDataEntries(Workspace->GetAssetDataEntriesView());
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([AssetDataEntries, Workspace=WeakWorkspace](const FAssetData& InAssetData)
			{
				if (AssetDataEntries.Contains(InAssetData))
				{
					return true;
				}
	
				return Workspace.Get() ? !Workspace.Get()->IsAssetSupported(InAssetData) : false;
			});

			return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
		}

		return SNullWidget::NullWidget;	
	
	}
	
private:
	TWeakObjectPtr<UWorkspace> WeakWorkspace;
	TSharedPtr<SPositiveActionButton> AddAssetButton;
	TSharedPtr<SActionButton> SaveAllButton;
};

void SWorkspaceView::Construct(const FArguments& InArgs, UWorkspace* InWorkspace, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor)
{
	Workspace = InWorkspace;

	FSceneOutlinerInitializationOptions InitOptions;
	{
		InitOptions.OutlinerIdentifier = TEXT("WorkspaceEditorOutliner");
		InitOptions.bShowHeaderRow = true;
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10));
		InitOptions.ColumnMap.Add(FWorkspaceOutlinerFileStateColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FWorkspaceOutlinerFileStateColumn(InSceneOutliner)); }), false));
		InitOptions.ColumnMap.Add(FSceneOutlinerSourceControlColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 100, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FSceneOutlinerSourceControlColumn(InSceneOutliner)); }), false));	
		InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this, WeakWorkspaceEditor=InWorkspaceEditor.ToWeakPtr()](SSceneOutliner* InOutliner)
		{
			OutlinerMode = new UE::Workspace::FWorkspaceOutlinerMode(UE::Workspace::FWorkspaceOutlinerMode(InOutliner, Workspace, WeakWorkspaceEditor));
			return OutlinerMode;
		});
	}
	SceneWorkspaceOutliner = SNew(SWorkspaceOutliner, InitOptions, Workspace)
		.CanSaveAll(InArgs._CanSaveAll)
		.OnSaveAll(InArgs._OnSaveAll);

	TWeakObjectPtr<UWorkspace> WeakWorkspace = Workspace;
	ChildSlot
	[
		SNew(SAssetDropTarget)
		.bSupportsMultiDrop(true)
		.OnAssetsDropped_Lambda([this](const FDragDropEvent& InEvent, TArrayView<FAssetData> InAssets)
		{
			FScopedTransaction Transaction(LOCTEXT("AddAssets", "Add assets to workspace"));
			Workspace->AddAssets(InAssets);
		})
		.OnAreAssetsAcceptableForDropWithReason_Lambda([this](TArrayView<FAssetData> InAssets, FText& OutText)
		{
			if (!Workspace->GetSchema()->SupportsMultipleAssets())
			{
				OutText = LOCTEXT("MultipleAssetsUnsupportedInWorkspace", "Multiple assets are not supported by this workspace");
				return false;
			}
			
			for(const FAssetData& Asset : InAssets)
			{
				if(Workspace->IsAssetSupported(Asset))
				{
					return true;
				}
			}

			OutText = LOCTEXT("AssetsUnsupportedInWorkspace", "Assets are not supported by this workspace");
			return false;
		})
		.Content()
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SceneWorkspaceOutliner.ToSharedRef()
			]
			+SOverlay::Slot()
			.Padding(32.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(
					LOCTEXT("EmptyWorkspaceTooltip", "No assets currently in this workspace. Use 'Add' button or drag and drop to add assets.")
				)
				.Justification(ETextJustify::Center)
				.AutoWrapText(true)
				.Visibility(
					TAttribute<EVisibility>::Create(
						TAttribute<EVisibility>::FGetter::CreateLambda([WeakWorkspace]()
						{
							if (UWorkspace* PinnedWorkspace = WeakWorkspace.Get())
							{
								if (!PinnedWorkspace->HasValidEntries())
								{
									return EVisibility::Visible;
								}
							}

							return EVisibility::Collapsed;
						})
					)
				)
			]
		]
	];
}

void SWorkspaceView::SelectObject(const UObject* InObject) const
{
	FWorkspaceOutlinerItemExport Export(InObject->GetFName(), InObject);

	FSceneOutlinerTreeItemPtr FoundItem = SceneWorkspaceOutliner->GetTreeItem(FSceneOutlinerTreeItemID(GetTypeHash(Export)));
	if(FoundItem.IsValid())
	{
		SceneWorkspaceOutliner->SetItemSelection(FoundItem, true, ESelectInfo::OnMouseClick);	// Not direct, so we get callbacks
	}
}

void SWorkspaceView::SelectExport(const FWorkspaceOutlinerItemExport& InExport) const
{
	FSceneOutlinerTreeItemPtr FoundItem = SceneWorkspaceOutliner->GetTreeItem(FSceneOutlinerTreeItemID(GetTypeHash(InExport)));
	if(FoundItem.IsValid())
	{
		SceneWorkspaceOutliner->SetItemSelection(FoundItem, true, ESelectInfo::OnMouseClick);	// Not direct, so we get callbacks
	}
}

void SWorkspaceView::GetWorkspaceExportData(FWorkspaceOutlinerItemExport& InOutPartialExport) const
{
	FSceneOutlinerTreeItemPtr FoundItem = SceneWorkspaceOutliner->GetTreeItem(FSceneOutlinerTreeItemID(GetTypeHash(InOutPartialExport)));
	if(FoundItem.IsValid())
	{
		if (const FWorkspaceOutlinerTreeItem* TreeItem = FoundItem->CastTo<FWorkspaceOutlinerTreeItem>())
		{
			InOutPartialExport = TreeItem->Export;
		}
	}
}

void SWorkspaceView::GetHierarchyAssetData(TArray<FAssetData>& OutAssetData) const
{
	if (OutlinerMode)
	{
		if (FWorkspaceOutlinerHierarchy* Hierarchy = static_cast<FWorkspaceOutlinerHierarchy*>(OutlinerMode->GetHierarchy()))
		{
			OutAssetData.Append(Hierarchy->ProcessedAssetData);
		}
	}	
}
}

#undef LOCTEXT_NAMESPACE
