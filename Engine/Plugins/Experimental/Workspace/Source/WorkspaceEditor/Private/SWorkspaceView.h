// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/SCompoundWidget.h"

struct FWorkspaceOutlinerItemExport;
class UWorkspace;
class UWorkSpaceAssetUserData;

namespace UE::Workspace
{
class SWorkspaceOutliner;
class IWorkspaceEditor;
class FWorkspaceOutlinerMode;

class SWorkspaceView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWorkspaceView) {}
		SLATE_ATTRIBUTE(bool, CanSaveAll)
		SLATE_EVENT(FOnClicked, OnSaveAll)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UWorkspace* InWorkspace, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor);

	void SelectObject(const UObject* InObject) const;
	void SelectExport(const FWorkspaceOutlinerItemExport& InExport) const;
	// Tries to lookup the FWorkspaceOutlinerItemExport path within the outliner, and copies over its TInstancedStruct<FWorkspaceOutlinerItemData> value
	void GetWorkspaceExportData(FWorkspaceOutlinerItemExport& InOutPartialExport) const;
	void GetHierarchyAssetData(TArray<FAssetData>& OutAssetData) const;
	
private:
	UWorkspace* Workspace = nullptr;
	FWorkspaceOutlinerMode* OutlinerMode = nullptr;
	TSharedPtr<SWorkspaceOutliner> SceneWorkspaceOutliner;
};

};