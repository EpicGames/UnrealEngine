// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorkspaceViewport.h"

#include "AdvancedPreviewScene.h"
#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "WorkspaceAssetViewportClient.h"
#include "WorkspaceViewportMenuContext.h"
#include "WorkspaceViewportSceneDescription.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "UE::Workspace::SWorkspaceViewport"

namespace UE::Workspace
{
	void SWorkspaceViewport::Construct(const FArguments& InArgs)
	{
		Client = InArgs._ViewportClient;
		AssetEditorToolkit = InArgs._AssetEditorToolkit;
		SceneDescription = InArgs._SceneDescription;
		
		PreviewAssetPath = InArgs._PreviewAssetPath;
		bIsPinned = InArgs._bIsPinned;
		OnPinnedClicked = InArgs._OnPinnedClicked;
		
		SEditorViewport::Construct(SEditorViewport::FArguments());
	}

	TSharedRef<FEditorViewportClient> SWorkspaceViewport::MakeEditorViewportClient()
	{
		return Client.ToSharedRef();
	}

	TSharedPtr<SWidget> SWorkspaceViewport::BuildViewportToolbar()
	{
		const FName ViewportToolbarName = "WorkspaceEditor.ViewportToolbar";

		if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
		{
			UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
				ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
			);
			ViewportToolbarMenu->StyleName = "ViewportToolbar";

			ViewportToolbarMenu->AddDynamicSection("Left", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					if (UWorkspaceViewportMenuContext* Context = InMenu->FindContext<UWorkspaceViewportMenuContext>())
					{
						InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitWidget(
							"PinPreview",
							SNew(SBox)
							[
								SNew(SCheckBox)
								.Style(FAppStyle::Get(), "DetailsView.SectionButton")
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
									.AutoWidth()
									[
										SNew(SImage)
										.Image_Lambda([Context]()
										{
											const bool bViewportPinned = Context->bIsPinned.Get();
											return FAppStyle::Get().GetBrush(bViewportPinned ? "Icons.Lock" : "Icons.Unlock");
										})
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										SNew(STextBlock)
										.ShadowOffset(FVector2D::UnitVector)
										.ShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f))
										.Text_Lambda([Context]()
										{
											return FText::Format(LOCTEXT("OverlayText", "Previewing {0}"), FText::FromString(Context->PreviewAssetPath.Get().GetAssetName()));
										})
									]
								]
								.IsChecked_Lambda([Context]()
								{
									return Context->bIsPinned.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
								.OnCheckStateChanged_Lambda([Context](ECheckBoxState InState)
								{
									Context->OnPinnedClicked.Execute();
								})
							],
							FText::GetEmpty()));
					
						InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitSubMenu(
							"SceneDescription",
							LOCTEXT("SceneDescription_Label", "Scene Settings"),
							LOCTEXT("SceneDescription_ToolTip", "Options to control the preview scene"),
							FNewToolMenuDelegate::CreateLambda([Context](UToolMenu* Submenu)
								{
									FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
													
									FDetailsViewArgs DetailsViewArgs;
									DetailsViewArgs.bAllowSearch = false;
									DetailsViewArgs.bAllowFavoriteSystem = false;
									DetailsViewArgs.bShowOptions = false;
									DetailsViewArgs.bShowObjectLabel = false;
									DetailsViewArgs.bHideSelectionTip = true;
									
									if (Context->SceneDescription)
									{
										const auto DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
										DetailsView->SetObject(Context->SceneDescription);

										Submenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitWidget(
											"Details",
											DetailsView,
											FText::GetEmpty(),
											true,
											false,
											true
										));
									}
								}),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.PreviewSceneSettings")));
					}
				}));

			{
				FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
				RightSection.Alignment = EToolMenuSectionAlign::Last;
	
				// Add the "Camera" submenu.
				RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowLensControls()));
	
				// Add the "View Modes" sub menu.
				{
					// Stay backward-compatible with the old viewport toolbar.
					{
						const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
						// Create our parent menu.
						if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
						{
							UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
						}
	
						// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
						UToolMenus::Get()->RegisterMenu("WaterWavesEditor.ViewportToolbar.ViewModes", ParentSubmenuName);
					}
	
					RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
				}
	
				RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
				RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
			}
		}

		FToolMenuContext ViewportToolbarContext;
		{
			UWorkspaceViewportMenuContext* WorkspaceViewportContext = ViewportToolbarContext.FindContext<UWorkspaceViewportMenuContext>();
			if (!WorkspaceViewportContext)
			{
				WorkspaceViewportContext = NewObject<UWorkspaceViewportMenuContext>();
				ViewportToolbarContext.AddObject(WorkspaceViewportContext);
			}

			WorkspaceViewportContext->bIsPinned = bIsPinned;
			WorkspaceViewportContext->PreviewAssetPath = PreviewAssetPath;
			WorkspaceViewportContext->OnPinnedClicked = OnPinnedClicked;
			WorkspaceViewportContext->SceneDescription = SceneDescription;
			
			ViewportToolbarContext.AppendCommandList(GetCommandList());

			// Add the UnrealEd viewport toolbar context.
			{
				UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
				ContextObject->Viewport = SharedThis(this);
				ContextObject->AssetEditorToolkit = AssetEditorToolkit;

				ViewportToolbarContext.AddObject(ContextObject);
			}
		}

		return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
	}

	TSharedRef<SEditorViewport> SWorkspaceViewport::GetViewportWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<FExtender> SWorkspaceViewport::GetExtenders() const
	{
		TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
		return Result;
	}

	void SWorkspaceViewport::OnFloatingButtonClicked()
	{
	}
	
	void SWorkspaceViewport::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(SceneDescription);
	}
}

#undef LOCTEXT_NAMESPACE