// Copyright Epic Games, Inc. All Rights Reserved.

#include "Workspace.h"

#include "ExternalPackageHelper.h"
#include "WorkspaceSchema.h"
#include "WorkspaceState.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Engine/ExternalAssetDependencyGatherer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Workspace)

REGISTER_ASSETDEPENDENCY_GATHERER(FExternalAssetDependencyGatherer, UWorkspace);

bool UWorkspace::AddAsset(const FAssetData& InAsset, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!GetSchema()->SupportsMultipleAssets() && !AssetEntries.IsEmpty())
	{
		ReportError(TEXT("UWorkspace::AddAssets: Schema does not support adding multiple assets."));
		return false;
	}
	
	if(!IsAssetSupported(InAsset))
	{
		ReportError(TEXT("UWorkspace::AddAsset: Unsupported asset supplied."));
		return false;
	}

	if(bSetupUndoRedo)
	{
		Modify(false);
	}

	int32 NewIndex = INDEX_NONE;

	const bool bAssetNotInWorkspace = !AssetEntries.ContainsByPredicate([&InAsset](const UWorkspaceAssetEntry* AssetEntry)-> bool
		{
			return AssetEntry->Asset.ToSoftObjectPath() == InAsset.ToSoftObjectPath();
		});

	if (bAssetNotInWorkspace)
	{
		// If we are a transient asset, dont use external packages
		const UObject* Asset = InAsset.GetAsset();
		check(Asset);
		if(!Asset->HasAnyFlags(RF_Transient))
		{
			UWorkspaceAssetEntry* NewEntry = NewObject<UWorkspaceAssetEntry>(this, UWorkspaceAssetEntry::StaticClass(), NAME_None, RF_Transactional);

			// Only use external packaging when this workspace is savable otherwise
			// we want assets housed in our transient workspace.
			if (GetSchema()->CanSaveWorkspace())
			{
				check(!HasAnyFlags(RF_Transient));
				FExternalPackageHelper::SetPackagingMode(NewEntry, this, true, false, PKG_None);
			}

			NewEntry->Asset = TSoftObjectPtr<UObject>(InAsset.GetSoftObjectPath());
			NewIndex = AssetEntries.Add(NewEntry);

			NewEntry->MarkPackageDirty();
		}
	}
	
	if(NewIndex != INDEX_NONE)
	{
		BroadcastModified();
	}

	return NewIndex != INDEX_NONE;
}

bool UWorkspace::AddAsset(UObject* InAsset, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAsset == nullptr)
	{
		ReportError(TEXT("UWorkspace::AddAsset: Invalid asset supplied."));
		return false;
	}

	return AddAsset(FAssetData(InAsset), bSetupUndoRedo, bPrintPythonCommand);
}

bool UWorkspace::AddAssets(TConstArrayView<FAssetData> InAssets, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAssets.Num() == 0)
	{
		ReportError(TEXT("UWorkspace::AddAssets: No assets supplied."));
		return false;
	}

	bool bAdded = false;
	{
		TGuardValue<bool> DisableNotifications(bSuspendNotifications, true);
		for(const FAssetData& Asset : InAssets)
		{
			bAdded |= AddAsset(Asset, bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	if(bAdded)
	{
		BroadcastModified();
	}

	return bAdded;
}

bool UWorkspace::AddAssets(const TArray<UObject*>& InAssets, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAssets.Num() == 0)
	{
		ReportError(TEXT("UWorkspace::AddAssets: No assets supplied."));
		return false;
	}

	bool bAdded = false;
	{
		TGuardValue<bool> DisableNotifications(bSuspendNotifications, true);
		for(UObject* Asset : InAssets)
		{
			bAdded |= AddAsset(FAssetData(Asset), bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	if(bAdded)
	{
		BroadcastModified();
	}

	return bAdded;
}

bool UWorkspace::RemoveAsset(const FAssetData& InAsset, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	const bool bIsRootLevelAsset = GetRootAsset() == InAsset.GetAsset();
	
	if (bIsRootLevelAsset)
	{
		ReportError(TEXT("UWorkspace::RemoveAsset: Cannot remove the root-level asset."));
		return false;
	}

	if(!IsAssetSupported(InAsset))
	{
		ReportError(TEXT("UWorkspace::RemoveAsset: Unsupported asset supplied."));
		return false;
	}

	if(bSetupUndoRedo)
	{
		Modify(false);
	}

	const int32 EntryIndex = AssetEntries.IndexOfByPredicate([&InAsset](const UWorkspaceAssetEntry* AssetEntry) -> bool
	{
		return AssetEntry->Asset.ToSoftObjectPath() == InAsset.ToSoftObjectPath();
	});

	if (EntryIndex != INDEX_NONE)
	{
		UWorkspaceAssetEntry* EntryToRemove = AssetEntries[EntryIndex];		
		verify(AssetEntries.Remove(EntryToRemove) == 1);
		EntryToRemove->MarkAsGarbage();
		EntryToRemove->MarkPackageDirty();
		
		BroadcastModified();
	}

	return EntryIndex != INDEX_NONE;
}

bool UWorkspace::RemoveAsset(UObject* InAsset, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAsset == nullptr)
	{
		ReportError(TEXT("UWorkspace::RemoveAsset: Invalid asset supplied."));
		return false;
	}

	return RemoveAsset(FAssetData(InAsset), bSetupUndoRedo, bPrintPythonCommand);
}

bool UWorkspace::RemoveAssets(TArray<UObject*> InAssets, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAssets.Num() == 0)
	{
		ReportError(TEXT("UWorkspace::RemoveAssets: No assets supplied."));
		return false;
	}

	bool bRemoved = false;
	{
		TGuardValue<bool> DisableNotifications(bSuspendNotifications, true);
		for(UObject* Asset : InAssets)
		{
			bRemoved |= RemoveAsset(FAssetData(Asset), bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	if(bRemoved)
	{
		BroadcastModified();
	}

	return bRemoved;
}

bool UWorkspace::RemoveAssets(TConstArrayView<FAssetData> InAssets, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InAssets.Num() == 0)
	{
		ReportError(TEXT("UWorkspace::RemoveAssets: No assets supplied."));
		return false;
	}

	bool bRemoved = false;
	{
		TGuardValue<bool> DisableNotifications(bSuspendNotifications, true);
		for(const FAssetData& Asset : InAssets)
		{
			bRemoved |= RemoveAsset(Asset, bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	if(bRemoved)
	{
		BroadcastModified();
	}

	return bRemoved;
}

bool UWorkspace::IsAssetSupported(const FAssetData& InAsset)
{
	const TConstArrayView<FTopLevelAssetPath> SupportedAssets = GetSchema()->GetSupportedAssetClassPaths();
	return SupportedAssets.IsEmpty() || SupportedAssets.Contains(InAsset.AssetClassPath);
}

UWorkspaceSchema* UWorkspace::GetSchema() const
{
	check(SchemaClass);
	return SchemaClass->GetDefaultObject<UWorkspaceSchema>();
}

void UWorkspace::LoadState() const
{
	GetState()->LoadFromJson(this);
}

void UWorkspace::SaveState() const
{
	GetState()->SaveToJson(this);
}

UWorkspaceState* UWorkspace::GetState() const
{
	if(State == nullptr)
	{
		State = NewObject<UWorkspaceState>(const_cast<UWorkspace*>(this));
	}

	return State;
}

void UWorkspace::BroadcastModified()
{
	EntriesAssetData.Reset();
	
	if(!bSuspendNotifications)
	{
		ModifiedDelegate.Broadcast(this);
	}
}

void UWorkspace::GetAssets(TArray<TObjectPtr<UObject>>& OutAssets) const
{
	for(const UWorkspaceAssetEntry* AssetEntry : AssetEntries)
	{
		if (AssetEntry)
		{
			if (UObject* Asset = AssetEntry->Asset.Get())
			{
				OutAssets.Add(Asset);
			}
		}
	}
}

TConstArrayView<FAssetData> UWorkspace::GetAssetDataEntriesView(bool bRefresh) const
{
	if (EntriesAssetData.Num() == 0 || bRefresh)
	{
		EntriesAssetData.Reset();
		const IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
		for (const UWorkspaceAssetEntry* AssetEntry : AssetEntries)
		{
			if (AssetEntry != nullptr)
			{
				EntriesAssetData.Add(AssetRegistry.GetAssetByObjectPath(AssetEntry->Asset.ToSoftObjectPath()));
			}
		}
	}
	
	return EntriesAssetData;
}

bool UWorkspace::HasValidEntries() const
{
	for(const UWorkspaceAssetEntry* AssetEntry : AssetEntries)
	{
		if (AssetEntry && !AssetEntry->Asset.IsNull())
		{
			return true;
		}
	}

	return false;
}

void UWorkspace::ReportError(const TCHAR* InMessage) const
{
#if WITH_EDITOR
	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, InMessage, TEXT(""));
#endif
}

void UWorkspace::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	BroadcastModified();
}

void UWorkspace::PostLoadExternalPackages()
{
	FExternalPackageHelper::LoadObjectsFromExternalPackages<UWorkspaceAssetEntry>(this, [this](UWorkspaceAssetEntry* InLoadedEntry)
	{
		check(IsValid(InLoadedEntry));
		AssetEntries.Add(InLoadedEntry);
	});

	EntriesAssetData.Reset();
}

void UWorkspace::Serialize(FArchive& Ar)
{
	UObject::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	const bool bIsDuplicating = (Ar.GetPortFlags() & PPF_Duplicate) != 0;
	if (bIsDuplicating)
	{
		Ar << AssetEntries;
		Ar << State;
	}
}

bool UWorkspace::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	FExternalPackageHelper::FRenameExternalObjectsHelperContext Context(this, Flags);
	return Super::Rename(NewName, NewOuter, Flags);
}

void UWorkspace::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	UObject::PreDuplicate(DupParams);	
	FExternalPackageHelper::DuplicateExternalPackages(this, DupParams);
}

void UWorkspace::PostLoad()
{
	Super::PostLoad();

	if(GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextMoveWorkspaces)
	{
		Guid = FGuid::NewGuid();
		SchemaClass = StaticLoadClass(UWorkspaceSchema::StaticClass(), nullptr, TEXT("/Script/UAFEditor.AnimNextWorkspaceSchema"));
	}

	if(GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextWorkspaceEntryConversion)
	{
		for (const TSoftObjectPtr<UObject>& SoftAsset : Assets_DEPRECATED)
		{
			if (UObject* Asset = SoftAsset.LoadSynchronous())
			{
				AddAsset(Asset);
			}
		}

		for (const TObjectPtr<UWorkspaceAssetEntry>& Entry : AssetEntries)
		{
			Entry->GetPackage()->SetDirtyFlag(true);	
		}

		Assets_DEPRECATED.Empty();
	}
	else
	{
		PostLoadExternalPackages();		
	}
}

void UWorkspace::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	Guid = FGuid::NewGuid();
}

UObject* UWorkspace::GetRootAsset() const
{
	if (!GetSchema()->HasPersistentRootAsset())
	{
		return nullptr;
	}

	check(!AssetEntries.IsEmpty());
	UWorkspaceAssetEntry* RootAssetEntry = AssetEntries[0];
	UObject* RootAsset = RootAssetEntry->Asset.LoadSynchronous();
	check(RootAsset);
	
	return RootAsset;
}
