// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"
#include "Templates/SubclassOf.h"
#include "WorkspaceAssetEntry.h"
#include "Workspace.generated.h"

struct FEditedDocumentInfo;
class UWorkspace;
class UWorkspaceState;
class UWorkspaceSchema;
class UWorkspaceFactory;
class UAssetDefinition_Workspace;
struct FInstancedStruct;

namespace UE::Workspace
{
	class FWorkspaceEditor;
	class FWorkspaceEditorModule;
	class SWorkspaceView;
	class FWorkspaceOutlinerHierarchy;
	class FWorkspaceOutlinerMode;
	class SWorkspaceOutliner;

	struct FWorkspaceOutliner;
}

namespace UE::UAF::Editor
{
	class FAssetWizard;
}

namespace UE::Workspace
{
	// A delegate for subscribing / reacting to workspace modifications.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorkspaceModified, UWorkspace* /* InWorkspace */);
}


// Workspace entry used to export to asset registry
USTRUCT()
struct FWorkspaceAssetRegistryExportEntry
{
	GENERATED_BODY()

	FWorkspaceAssetRegistryExportEntry() = default;
	
	FWorkspaceAssetRegistryExportEntry(const FSoftObjectPath& InAsset)
		: Asset(InAsset)
	{}

	UPROPERTY()
	FSoftObjectPath Asset;
};

// Workspace used to export to asset registry
USTRUCT()
struct FWorkspaceAssetRegistryExports
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FWorkspaceAssetRegistryExportEntry> Assets;
};

UCLASS(MinimalAPI)
class UWorkspace : public UObject
{
	GENERATED_BODY()

	friend class UE::Workspace::FWorkspaceEditor;
	friend class UE::Workspace::FWorkspaceEditorModule;
	friend class UE::Workspace::SWorkspaceView;
	friend struct UE::Workspace::FWorkspaceOutliner;
	friend class UE::Workspace::FWorkspaceOutlinerHierarchy;
	friend class UE::Workspace::FWorkspaceOutlinerMode;
	friend class UE::Workspace::SWorkspaceOutliner;
	
	friend class UWorkspaceFactory;
	friend class UWorkspaceAssetEditor;
	friend class UAssetDefinition_Workspace;
	friend class UWorkspaceState;

	friend class UE::UAF::Editor::FAssetWizard;
	
	friend struct FUAFWorkspaceTests;

	// Adds an asset to the workspace
	// @return true if the asset was added
	UFUNCTION(BlueprintCallable, Category = "Workspace")
	WORKSPACEEDITOR_API bool AddAsset(UObject* InAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	bool AddAsset(const FAssetData& InAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Adds assets to the workspace
	// @return true if an asset was added
	UFUNCTION(BlueprintCallable, Category = "Workspace")
	bool AddAssets(const TArray<UObject*>& InAssets, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	bool AddAssets(TConstArrayView<FAssetData> InAssets, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Removes an asset from the workspace
	// @return true if the asset was removed
	UFUNCTION(BlueprintCallable, Category = "Workspace")
	WORKSPACEEDITOR_API bool RemoveAsset(UObject* InAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	bool RemoveAsset(const FAssetData& InAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Removes assets from the workspace
	// @return true if the asset was removed
	UFUNCTION(BlueprintCallable, Category = "Workspace")
	bool RemoveAssets(TArray<UObject*> InAssets, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	bool RemoveAssets(TConstArrayView<FAssetData> InAssets, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	bool IsAssetSupported(const FAssetData& InAsset);

	UWorkspaceSchema* GetSchema() const;

	void LoadState() const;
	void SaveState() const;
	UWorkspaceState* GetState() const;

	void ReportError(const TCHAR* InMessage) const;

	void BroadcastModified();

	// Returns the "root asset" that this workspace was created for.
	// Returns nullptr if the schema does not support a root asset.
	// Root assets cannot be removed from the workspace.
	UObject* GetRootAsset() const;

	// Returns all contained Asset objects
	void GetAssets(TArray<TObjectPtr<UObject>>& OutAssets) const;
	// Returns all contained AssetData entries
	TConstArrayView<FAssetData> GetAssetDataEntriesView(bool bRefresh = false) const;

	// Returns whether or not the workspace contains _any_ valid UWorkspaceAssetEntry
	bool HasValidEntries() const;

	// UObject interface
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	virtual bool IsEditorOnly() const override { return true; }
	virtual void Serialize(FArchive& Ar) override;
	virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;	
	virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;

	void PostLoadExternalPackages();
	
	// All of the assets referenced by this workspace
	UPROPERTY()
	TArray<TSoftObjectPtr<UObject>> Assets_DEPRECATED;

	// All of the assets in-directly referenced by this workspace
	UPROPERTY(transient)
	TArray<TObjectPtr<UWorkspaceAssetEntry>> AssetEntries;
	
	mutable TArray<FAssetData> EntriesAssetData;

	// Schema for this workspace
	UPROPERTY(AssetRegistrySearchable)
	TSubclassOf<UWorkspaceSchema> SchemaClass;

	// State of the workspace, persisted to json
	UPROPERTY(Transient)
	mutable TObjectPtr<UWorkspaceState> State = nullptr;

	// Guid for persistent identification of this workspace
	UPROPERTY()
	FGuid Guid;

	// Delegate to subscribe to modifications
	UE::Workspace::FOnWorkspaceModified ModifiedDelegate;

	bool bSuspendNotifications = false;
};
