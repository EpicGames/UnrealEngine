// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetDefinition.h"
#include "WorkspaceEditor.h"
#include "WorkspaceSchema.h"
#include "WorkspaceAssetEditor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceAssetDefinition)

#define LOCTEXT_NAMESPACE "AssetDefinition_Workspace"

FText UAssetDefinition_Workspace::GetAssetDisplayName() const
{
	return LOCTEXT("Workspace", "Workspace");
}

FText UAssetDefinition_Workspace::GetAssetDisplayName(const FAssetData& AssetData) const
{
	FString TagValue;
	if(AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UWorkspace, SchemaClass), TagValue))
	{
		if(UClass* SchemaClass = FindObject<UClass>(nullptr, *TagValue, EFindObjectFlags::ExactClass))
		{
			FText DisplayName = SchemaClass->GetDefaultObject<UWorkspaceSchema>()->GetDisplayName();
			if(!DisplayName.IsEmpty())
			{
				return DisplayName;
			}
		}
	}
	
	return GetAssetDisplayName();
}

EAssetCommandResult UAssetDefinition_Workspace::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UWorkspace* Asset : OpenArgs.LoadObjects<UWorkspace>())
	{
		if (!Asset->GetSchema()->CanSaveWorkspace())
		{
			// Attempting to open a workspace asset that is no longer savable as an asset
			FNotificationInfo ErrorNotification(LOCTEXT("CannotOpenWorkspaceAsset", "Workspace assets of this schema can no longer be saved as assets. Please delete this asset."));
			ErrorNotification.bFireAndForget = true;
			ErrorNotification.bUseLargeFont = false;
			ErrorNotification.ExpireDuration = 3.0f;
			
            FSlateNotificationManager::Get().AddNotification(ErrorNotification)->SetCompletionState(SNotificationItem::CS_Fail);
			UE_LOGF(LogAnimation, Error, "Workspace assets of this schema can no longer be saved as assets. Please delete this asset.");
			
			return EAssetCommandResult::Handled;
		}
	
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UWorkspaceAssetEditor* AssetEditor = NewObject<UWorkspaceAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->SetObjectToEdit(Asset);
		AssetEditor->Initialize();
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
