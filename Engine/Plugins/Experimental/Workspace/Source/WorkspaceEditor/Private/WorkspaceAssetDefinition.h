// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Workspace.h"
#include "AssetDefinitionDefault.h"
#include "WorkspaceAssetDefinition.generated.h"

#define LOCTEXT_NAMESPACE "AssetDefinitions"

UCLASS()
class UAssetDefinition_Workspace : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(170,96,48)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UWorkspace::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Misc) };
		return Categories;
	}
	virtual bool ShouldSaveExternalPackages() const override { return true; }
};

#undef LOCTEXT_NAMESPACE