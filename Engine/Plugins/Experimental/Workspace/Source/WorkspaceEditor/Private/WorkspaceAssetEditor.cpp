// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetEditor.h"
#include "WorkspaceEditor.h"
#include "WorkspaceSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceAssetEditor)

void UWorkspaceAssetEditor::GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit)
{
	if (ObjectToEdit)
	{
		if (const UWorkspaceSchema* Schema = ObjectToEdit->GetSchema())
		{
			if (Schema->CanSaveWorkspace())
			{
				OutObjectsToEdit.Add(ObjectToEdit);
			}
			else
			{
				// Transient workspace outputs its assets
				TArray<TObjectPtr<UObject>> AssetObjects;
				ObjectToEdit->GetAssets(AssetObjects);
				OutObjectsToEdit.Append(AssetObjects);
			}
		}
	}
}

void UWorkspaceAssetEditor::SetObjectToEdit(UWorkspace* InWorkspace)
{
	ObjectToEdit = InWorkspace;
}

UWorkspace* UWorkspaceAssetEditor::GetObjectToEdit()
{
	return ObjectToEdit;
}

TSharedPtr<FBaseAssetToolkit> UWorkspaceAssetEditor::CreateToolkit()
{
	return MakeShared<UE::Workspace::FWorkspaceEditor>(this);
}

