// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"
#include "Workspace.h"
#include "WorkspaceAssetEditor.generated.h"

class FBaseAssetToolkit;

UCLASS(Transient)
class UWorkspaceAssetEditor : public UAssetEditor
{
	GENERATED_BODY()
public:
	void SetObjectToEdit(UWorkspace* InWorkspace);
	UWorkspace* GetObjectToEdit();
	
protected:
	/** For standalone workspace, the workspace itself will be returned otherwise the underlying assets will be returned */
	virtual void GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:
	UPROPERTY()
	TObjectPtr<UWorkspace> ObjectToEdit;
};

