// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetEntry.h"
#include "Workspace.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceAssetEntry)

const FName UWorkspaceAssetEntry::ExportsAssetRegistryTag = TEXT("Exports");

bool UWorkspaceAssetEntry::IsAsset() const
{
	// Entries are considered assets to allow using the asset logic for save dialogs, etc.
	// Also, they return true even if pending kill, in order to show up as deleted in these dialogs.
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_Transient | RF_ClassDefaultObject);
}

void UWorkspaceAssetEntry::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	UObject::GetAssetRegistryTags(Context);
	Context.AddTag(FAssetRegistryTag(UWorkspaceAssetEntry::ExportsAssetRegistryTag, Asset.GetUniqueID().ToString(), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag, *Asset.GetAssetName(), FAssetRegistryTag::TT_Hidden));
}
