// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "WorkspaceAssetEntry.generated.h"

class FAssetRegistryTagsContext;

UCLASS()
class UWorkspaceAssetEntry : public UObject
{
	GENERATED_BODY()
public:	
	WORKSPACEEDITOR_API static const FName ExportsAssetRegistryTag;

	virtual bool IsAsset() const override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	UPROPERTY()
	TSoftObjectPtr<UObject> Asset;	
};
