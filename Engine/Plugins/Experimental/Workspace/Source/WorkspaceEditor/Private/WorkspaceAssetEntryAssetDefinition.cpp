// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetEntryAssetDefinition.h"
#include "WorkspaceAssetEntry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceAssetEntryAssetDefinition)

#define LOCTEXT_NAMESPACE "AssetDefinition_WorkspaceAssetEntry"

FText UAssetDefinition_WorkspaceAssetEntry::GetAssetDisplayName() const
{
	return LOCTEXT("WorkSpaceAssetEntry", "Workspace Asset Entry");
}

FLinearColor UAssetDefinition_WorkspaceAssetEntry::GetAssetColor() const
{
	return FLinearColor(FColor(64,64,64));
}

TSoftClassPtr<> UAssetDefinition_WorkspaceAssetEntry::GetAssetClass() const
{
	return UWorkspaceAssetEntry::StaticClass();
}

FText UAssetDefinition_WorkspaceAssetEntry::GetObjectDisplayNameText(UObject* Object) const
{
	const UWorkspaceAssetEntry* Entry = CastChecked<UWorkspaceAssetEntry>(Object);
	return FText::FromString(Entry->Asset.GetAssetName());
}

#undef LOCTEXT_NAMESPACE // "AssetDefinition_WorkspaceAssetEntry"
