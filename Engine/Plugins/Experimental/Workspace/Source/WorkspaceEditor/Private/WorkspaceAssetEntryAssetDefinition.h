// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"
#include "UObject/Object.h"
#include "WorkspaceAssetEntryAssetDefinition.generated.h"

UCLASS()
class UAssetDefinition_WorkspaceAssetEntry  : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
};
