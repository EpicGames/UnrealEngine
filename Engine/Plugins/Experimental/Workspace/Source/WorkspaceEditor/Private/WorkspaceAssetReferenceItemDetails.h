// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IWorkspaceOutlinerItemDetails.h"
#include "WorkspaceEditorModule.h"
#include "Styling/AppStyle.h"

class UUAFRigVMAssetEditorData;
enum class EAnimNextEditorDataNotifType : uint8;

namespace UE::Workspace
{
class FWorkspaceAssetReferenceOutlinerItemDetails : public IWorkspaceOutlinerItemDetails
{
public:
	virtual ~FWorkspaceAssetReferenceOutlinerItemDetails() override = default;

	static TSharedPtr<IWorkspaceOutlinerItemDetails> GetInnerDetails(const FWorkspaceOutlinerItemExport& Export)
	{
		TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = nullptr;
		const FWorkspaceOutlinerAssetReferenceItemData& Data = Export.GetData().Get<FWorkspaceOutlinerAssetReferenceItemData>();
		if(Data.ReferredExport.HasData())
		{
			const FOutlinerItemDetailsId Id = Data.ReferredExport.GetData().GetScriptStruct()->GetFName();		
			SharedFactory = FWorkspaceEditorModule::GetOutlinerItemDetails(Id);
		}

		return SharedFactory;
	}

	static const FWorkspaceOutlinerItemExport& GetInnerExport(const FWorkspaceOutlinerItemExport& Export)
	{
		FWorkspaceOutlinerItemExport InnerExport = Export;
		check(Export.HasData());
		const FWorkspaceOutlinerAssetReferenceItemData& Data = Export.GetData().Get<FWorkspaceOutlinerAssetReferenceItemData>();
		return Data.ReferredExport;
	}

	virtual FString GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const override
	{
		if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = GetInnerDetails(Export))
		{
			return GetInnerDetails(Export)->GetDisplayString(GetInnerExport(Export));
		}

		return Export.GetFirstAssetPath().GetAssetName();
	}
	
	virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override
	{
		const FSlateBrush* Icon = nullptr;
		{
			if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = GetInnerDetails(Export))
			{
				const FWorkspaceOutlinerItemExport InnerExport = GetInnerExport(Export);
				Icon = SharedFactory->GetItemIcon(InnerExport);
			}
		}
				
		return Icon ? Icon : FAppStyle::GetBrush(TEXT("GenericLink"));
	}
	
	virtual FSlateColor GetItemColor(const FWorkspaceOutlinerItemExport& Export) const override
	{
		const FWorkspaceOutlinerAssetReferenceItemData& Data = Export.GetData().Get<FWorkspaceOutlinerAssetReferenceItemData>();	
		
		FSlateColor InnerColor = FSlateColor::UseSubduedForeground();
		if (const TSharedPtr<IWorkspaceOutlinerItemDetails> SharedFactory = GetInnerDetails(Export))
		{
			const FWorkspaceOutlinerItemExport InnerExport = GetInnerExport(Export);
			InnerColor = SharedFactory->GetItemColor(InnerExport);
		}
		
		return Data.bRecursiveReference && Data.bShouldExpandReference ? FLinearColor::Red : InnerColor;
	}	

	virtual bool IsExpandedByDefault() const override
	{
		return false;
	}	
};

}
