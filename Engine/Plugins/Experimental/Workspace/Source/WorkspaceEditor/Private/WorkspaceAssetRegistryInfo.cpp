// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetRegistryInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceAssetRegistryInfo)

FWorkspaceOutlinerItemExport& FWorkspaceOutlinerItemExport::GetResolvedExport()
{
	if (HasData() && Data.GetScriptStruct() == FWorkspaceOutlinerAssetReferenceItemData::StaticStruct())
	{
		return Data.GetMutable<FWorkspaceOutlinerAssetReferenceItemData>().ReferredExport;
	}

	return *this;
}

const FWorkspaceOutlinerItemExport& FWorkspaceOutlinerItemExport::GetResolvedExport() const
{
	if (HasData() && Data.GetScriptStruct() == FWorkspaceOutlinerAssetReferenceItemData::StaticStruct())
	{
		return Data.Get<FWorkspaceOutlinerAssetReferenceItemData>().ReferredExport;
	}

	return *this;
}
