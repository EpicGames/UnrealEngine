// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceAssetViewportClient.h"

#include "PreviewScene.h"

namespace UE::Workspace
{
	void FWorkspaceAssetViewportClient::Tick(float DeltaTime)
	{
		if (PreviewScene)
		{
			PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaTime);
		}

		FEditorViewportClient::Tick(DeltaTime);
	}
}