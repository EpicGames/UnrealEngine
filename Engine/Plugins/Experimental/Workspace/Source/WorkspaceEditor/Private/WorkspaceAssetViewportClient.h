// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"

namespace UE::Workspace
{
	class FWorkspaceAssetViewportClient : public FEditorViewportClient
	{
	public:
		FWorkspaceAssetViewportClient(FEditorModeTools* InModeTools, const TSharedPtr<FAdvancedPreviewScene>& InPreviewScene)
			: FEditorViewportClient(InModeTools, InPreviewScene.Get())
			, AdvancedPreviewScene(InPreviewScene)
		{
		}
		
		// FEditorViewportClient interface
		virtual void Tick(float DeltaTime) override;
		
		TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene = nullptr;
	};
}