// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceEditor.h"
#include "Workspace.h"
#include "AssetDocumentSummoner.h"
#include "ExternalPackageHelper.h"
#include "IStructureDetailsView.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceAssetEditor.h"
#include "WorkspaceState.h"
#include "WorkspaceDocumentState.h"
#include "WorkspaceEditorModule.h"
#include "SWorkspaceView.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ToolMenuContext.h"
#include "WorkspaceEditorCommands.h"
#include "SWorkspaceTabWrapper.h"
#include "Dialogs/Dialogs.h"
#include "ToolMenus.h"
#include "AssetEditorModeManager.h"
#include "Selection.h"
#include "EditorModeManager.h"
#include "SAssetEditorViewport.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SGraphDocument.h"
#include "WorkspaceSchema.h"
#include "WorkspaceTabPayload.h"
#include "AdvancedPreviewScene.h"
#include "ContextObjectStore.h"
#include "SWorkspaceViewport.h"
#include "WorkspaceAssetViewportClient.h"
#include "WorkspaceViewportController.h"
#include "Viewports.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "WorkspaceEditor"

namespace UE::Workspace
{

static bool GOutputWorkspaceStateToCrashContext = false;
static FAutoConsoleVariableRef GOutputWorkspaceStateToCrashContextVar(TEXT("Workspace.OutputWorkspaceStateToCrashContext"), GOutputWorkspaceStateToCrashContext, TEXT("When enabled will output the state of all workspace editor instances to the crash context"));

FWorkspaceEditorSelectionScope::FWorkspaceEditorSelectionScope(const TSharedPtr< IWorkspaceEditor>& InWorkspaceEditor) : WeakWorkspaceEditor(InWorkspaceEditor)
{
	const TSharedPtr<FWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<FWorkspaceEditor>(InWorkspaceEditor);
	++SharedWorkspaceEditor->SelectionScopeDepth;
}

FWorkspaceEditorSelectionScope::~FWorkspaceEditorSelectionScope()
{
	if (const TSharedPtr<FWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<FWorkspaceEditor>(WeakWorkspaceEditor.Pin()))
	{
		--SharedWorkspaceEditor->SelectionScopeDepth;
		check(SharedWorkspaceEditor->SelectionScopeDepth >= 0);
		
		if (SharedWorkspaceEditor->SelectionScopeDepth == 0)
		{
			SharedWorkspaceEditor->bSelectionScopeCleared = false;
		}
	}
}

namespace WorkspaceModes
{
	const FName WorkspaceEditor("WorkspaceEditorMode");
}

namespace WorkspaceTabs
{
	const FName Details("DetailsTab");
	const FName WorkspaceView("WorkspaceView");
	const FName TopLeftDocumentArea("TopLeftDocumentArea");
	const FName BottomLeftDocumentArea("BottomLeftDocumentArea");
	const FName TopMiddleDocumentArea("TopMiddleDocumentArea");
	const FName BottomMiddleDocumentArea("BottomMiddleDocumentArea");
	const FName TopRightDocumentArea("TopRightDocumentArea");
	const FName BottomRightDocumentArea("BottomRightDocumentArea");
	const FName RightSidebarDocumentArea("RightSidebarDocumentArea");
}

const FName WorkspaceAppIdentifier("WorkspaceEditor");

FWorkspaceEditor::FWorkspaceEditor(UWorkspaceAssetEditor* InOwningAssetEditor) : IWorkspaceEditor(InOwningAssetEditor)
{
	Workspace = Cast<UWorkspaceAssetEditor>(OwningAssetEditor)->GetObjectToEdit();
	bCheckDirtyOnAssetSave = true;
}

void FWorkspaceEditor::CreateWidgets()
{
	DocumentManager = MakeShared<FDocumentTracker>(NAME_None);
	DocumentManager->Initialize(SharedThis(this));

	FBaseAssetToolkit::CreateWidgets();

	FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");

	// Build document summoners for each workspace layout area
	const TSharedRef<FAssetDocumentSummoner> TopLeftAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::TopLeftDocumentArea, SharedThis(this));
	TopLeftAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::TopLeftDocumentArea));
	DocumentManager->RegisterDocumentFactory(TopLeftAssetDocumentSummoner);

	const TSharedRef<FAssetDocumentSummoner> BottomLeftAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::BottomLeftDocumentArea, SharedThis(this));
	BottomLeftAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::BottomLeftDocumentArea));
	DocumentManager->RegisterDocumentFactory(BottomLeftAssetDocumentSummoner);

	constexpr bool bAllowUnsupportedClasses = true;
	const TSharedRef<FAssetDocumentSummoner> TopMiddleAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::TopMiddleDocumentArea, SharedThis(this), bAllowUnsupportedClasses);
	TopMiddleAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::TopMiddleDocumentArea));
	DocumentManager->RegisterDocumentFactory(TopMiddleAssetDocumentSummoner);

	const TSharedRef<FAssetDocumentSummoner> BottomMiddleAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::BottomMiddleDocumentArea, SharedThis(this));
	BottomMiddleAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::BottomMiddleDocumentArea));
	DocumentManager->RegisterDocumentFactory(BottomMiddleAssetDocumentSummoner);

	const TSharedRef<FAssetDocumentSummoner> TopRightAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::TopRightDocumentArea, SharedThis(this));
	TopRightAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::TopRightDocumentArea));
	DocumentManager->RegisterDocumentFactory(TopRightAssetDocumentSummoner);

	const TSharedRef<FAssetDocumentSummoner> BottomRightAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::BottomRightDocumentArea, SharedThis(this));
	BottomRightAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::BottomRightDocumentArea));
	DocumentManager->RegisterDocumentFactory(BottomRightAssetDocumentSummoner);

	const TSharedRef<FAssetDocumentSummoner> RightSidebarAssetDocumentSummoner = MakeShared<FAssetDocumentSummoner>(WorkspaceTabs::RightSidebarDocumentArea, SharedThis(this));
	RightSidebarAssetDocumentSummoner->SetAllowedClassPaths(WorkspaceEditorModule.GetAllowedObjectTypesForArea(WorkspaceTabs::RightSidebarDocumentArea));
	DocumentManager->RegisterDocumentFactory(RightSidebarAssetDocumentSummoner);

	check(DetailsView.IsValid());
	WorkspaceEditorModule.ApplyWorkspaceDetailsCustomization(StaticCastWeakPtr<IWorkspaceEditor>(this->AsWeak()), DetailsView);

	StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_WorkspaceEditor_Layout_v5")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.25f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(false)
					->AddTab(WorkspaceTabs::TopLeftDocumentArea, ETabState::ClosedTab)
					->AddTab(WorkspaceTabs::WorkspaceView, ETabState::OpenedTab)
					->SetExtensionId(WorkspaceTabs::TopLeftDocumentArea)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(false)
					->AddTab(WorkspaceTabs::BottomLeftDocumentArea, ETabState::ClosedTab)
					->SetExtensionId(WorkspaceTabs::BottomLeftDocumentArea)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.5f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.75f)
					->SetHideTabWell(false)
					->AddTab(WorkspaceTabs::TopMiddleDocumentArea, ETabState::ClosedTab)
					->SetExtensionId(WorkspaceTabs::TopMiddleDocumentArea)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->SetHideTabWell(false)
					->AddTab(WorkspaceTabs::BottomMiddleDocumentArea, ETabState::ClosedTab)
					->SetExtensionId(WorkspaceTabs::BottomMiddleDocumentArea)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.25f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(false)
					->AddTab(FBaseAssetToolkit::ViewportTabID, ETabState::OpenedTab)
					->AddTab(WorkspaceTabs::TopRightDocumentArea, ETabState::ClosedTab)
					->SetExtensionId(WorkspaceTabs::TopRightDocumentArea)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(false)
					->AddTab(WorkspaceTabs::BottomRightDocumentArea, ETabState::ClosedTab)
					->AddTab(FBaseAssetToolkit::DetailsTabID, ETabState::OpenedTab)
					->SetExtensionId(WorkspaceTabs::BottomRightDocumentArea)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(WorkspaceTabs::RightSidebarDocumentArea, ETabState::SidebarTab, ESidebarLocation::Right, /*SidebarSizeCoefficient*/ 0.2f)
			)
		)
	);

	WorkspaceEditorModule.OnExtendTabs().Broadcast(*LayoutExtender.Get(), StaticCastSharedRef<UE::Workspace::IWorkspaceEditor>(AsShared()));
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender.Get());

	WorkspaceView = SNew(SWorkspaceView, Workspace, StaticCastSharedRef<UE::Workspace::IWorkspaceEditor>(AsShared()))
		.CanSaveAll_Lambda([this]()
		{
			return AreAssetEntriesModified();
		})
		.OnSaveAll_Lambda([this]()
		{
			SaveAssetEntries();
			return FReply::Handled();
		});

	OnOutlinerSelectionChangedDelegate.AddSP(this, &FWorkspaceEditor::HandleOutlinerSelectionChanged);
	
	BindCommands();
}

void FWorkspaceEditor::PostInitAssetEditor()
{
#if WITH_ADDITIONAL_CRASH_CONTEXTS
	if (GOutputWorkspaceStateToCrashContext)
	{
		FGenericCrashContext::OnAdditionalCrashContextDelegate().AddSP(this, &FWorkspaceEditor::OnProvideAdditionalCrashContext);
	}
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

	if (UContextObjectStore* ContextObjectStore = GetEditorModeManager().GetInteractiveToolsContext()->ContextObjectStore)
	{
		UAssetEditorToolkitMenuContext* AssetEditorContext = ContextObjectStore->FindContext<UAssetEditorToolkitMenuContext>();
		if (!AssetEditorContext)
		{
			AssetEditorContext = NewObject<UAssetEditorToolkitMenuContext>();
			AssetEditorContext->Toolkit = AsShared();
			ContextObjectStore->AddContextObject(AssetEditorContext);
		}
	}
	
	ExtendMenu();
	ExtendToolbar();
	
	Workspace->LoadState();
	GetSchema()->OnLoadWorkspaceState(SharedThis(this), Workspace->GetState()->UserState);
	RestoreEditedObjectState();

	RegenerateMenusAndToolbars();

	// Default pin the document that was opened (the root-most asset)
	ViewportPinnedAssetPath = GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();
}

void FWorkspaceEditor::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	TSharedPtr<FWorkspaceEditorModeUILayer> ModeUILayer; 
	ModeUILayers.RemoveAndCopyValue(Toolkit->GetToolkitFName(), ModeUILayer);
	if(ModeUILayer)
	{
		ModeUILayer->OnToolkitHostingFinished(Toolkit);
	}
		
	UToolMenus::UnregisterOwner(&(*Toolkit));	
	HostedToolkits.Remove( Toolkit );
		
	RegenerateMenusAndToolbars();
}

void FWorkspaceEditor::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ensure(!ModeUILayers.Contains(Toolkit->GetToolkitFName()));

	TSharedPtr<FWorkspaceEditorModeUILayer> ModeUILayer = MakeShared<FWorkspaceEditorModeUILayer>(ToolkitHost.Pin().Get());
	ModeUILayer->SetModeMenuCategory(EditorMenuCategory);

	// Actually re-use the main toolbar rather than a secondary, which also requires appending the UI layer commands
	ModeUILayer->SetSecondaryModeToolbarName(GetToolMenuToolbarName());	
	ToolkitCommands->Append(ModeUILayer->GetModeCommands());

	ModeUILayer->OnToolkitHostingStarted(Toolkit);

	ModeUILayers.Add(Toolkit->GetToolkitFName(), ModeUILayer);	
	HostedToolkits.Add(Toolkit);

	RegenerateMenusAndToolbars();
}

void FWorkspaceEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Workspace);
	Collector.AddReferencedObject(SceneDescription);
}

void FWorkspaceEditor::RestoreEditedObjectState()
{
	UWorkspaceState* State = Workspace->GetState();
	for (const TInstancedStruct<FWorkspaceDocumentState>& DocumentState : State->DocumentStates)
	{
		if (UObject* Object = DocumentState.Get().Object.TryLoad())
		{
			const FWorkspaceOutlinerItemExport& Export = DocumentState.Get().Export;
			if(const TSharedPtr<SDockTab> DockTab = OpenDocument(Object, Export, FDocumentTracker::RestorePreviousDocument))
			{
				const TSharedRef<SWorkspaceTabWrapper> TabWrapper = StaticCastSharedRef<SWorkspaceTabWrapper>(DockTab->GetContent());	
				FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
				const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({Export, Object});
				if(DocumentArgs != nullptr && DocumentArgs->OnSetDocumentState.IsBound())
				{
					DocumentArgs->OnSetDocumentState.Execute(FWorkspaceEditorContext(SharedThis(this), { Export, Object }), TabWrapper->GetContent(), DocumentState);
				}
			}
		}
	}
}

void FWorkspaceEditor::SaveEditedObjectState() const
{
	// Clear edited document state
	UWorkspaceState* State = Workspace->GetState();
	State->DocumentStates.Empty();

	// Ask all open documents to save their state, which will update edited documents
	DocumentManager->SaveAllState();

	// Persist state
	Workspace->GetSchema()->OnSaveWorkspaceState(SharedThis(const_cast<FWorkspaceEditor*>(this)), Workspace->GetState()->UserState);
	Workspace->SaveState();
}

TSharedPtr<SDockTab> FWorkspaceEditor::OpenDocument(const UObject* InForObject, FDocumentTracker::EOpenDocumentCause InCause)
{
	FWorkspaceOutlinerItemExport Export(InForObject->GetFName(), InForObject);
	return OpenDocument(InForObject, Export, InCause);
}

TSharedPtr<SDockTab> FWorkspaceEditor::OpenDocument(const UObject* InForObject, const FWorkspaceOutlinerItemExport& InExport, FDocumentTracker::EOpenDocumentCause InCause)
{
	const TSharedRef<FTabPayload_WorkspaceDocument> Payload = FTabPayload_WorkspaceDocument::Make(InForObject, InExport);
	const bool bIsSupportedDocument = DocumentManager->FindSupportingFactory(Payload).IsValid();
	if (bIsSupportedDocument)
	{
		const bool bIsRootAsset = Workspace->GetRootAsset() == InForObject;
		
		if (bIsRootAsset || Workspace->GetSchema()->SupportsMultipleAssets())
		{
			// Only associate this workspace editor with the root level asset
			// Referenced assets are opened as a 'preview' but should not be used in AddEditingObject
			// as it will prevents opening that asset in a new workspace editor as a root-level asset
			AddEditingObject(const_cast<UObject*>(InForObject));
		}
	}
	
	TSharedPtr<SDockTab> NewTab = DocumentManager->OpenDocument(Payload, InCause);

	if(InCause != FDocumentTracker::RestorePreviousDocument)
	{
		const FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
		const FObjectDocumentArgs* DocumentArgs = WorkspaceEditorModule.FindObjectDocumentType({InExport, const_cast<UObject*>(InForObject)});
		if(DocumentArgs != nullptr && DocumentArgs->OnGetDocumentState.IsBound())
		{
			const TSharedRef<SWorkspaceTabWrapper> TabWrapper = StaticCastSharedRef<SWorkspaceTabWrapper>(NewTab->GetContent());
			RecordDocumentState(DocumentArgs->OnGetDocumentState.Execute(FWorkspaceEditorContext(SharedThis(this), { InExport, const_cast<UObject*>(InForObject) }), TabWrapper->GetContent()));
		}
		else
		{
			RecordDocumentState(TInstancedStruct<FWorkspaceDocumentState>::Make(InForObject, InExport));
		}
	}

	return NewTab;
}

void FWorkspaceEditor::OpenAssets(TConstArrayView<FAssetData> InAssets)
{
	for(const FAssetData& Asset : InAssets)
	{
		if(const UObject* LoadedAsset = Asset.GetAsset())
		{
			OpenDocument(LoadedAsset, FDocumentTracker::EOpenDocumentCause::NavigatingCurrentDocument);
		}
	}

	if(InAssets.Num() > 0)
	{
		if(UObject* LoadedAsset = InAssets.Last().GetAsset())
		{
			WorkspaceView->SelectObject(LoadedAsset);
		}
	}
}

void FWorkspaceEditor::OpenExports(TConstArrayView<FWorkspaceOutlinerItemExport> InExports, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride)
{
	const FDocumentTracker::EOpenDocumentCause OpenCause = OpenCauseOverride.Get(FDocumentTracker::EOpenDocumentCause::NavigatingCurrentDocument); 

	TArray<const FWorkspaceOutlinerItemExport> OpenedExports;
	for(const FWorkspaceOutlinerItemExport& Export : InExports)
	{
		FWorkspaceOutlinerItemExport QualifiedExport = Export; 
		WorkspaceView->GetWorkspaceExportData(QualifiedExport);
		
		if(UObject* LoadedObject = QualifiedExport.GetFirstValidObjectPath().TryLoad())
		{
			OpenDocument(LoadedObject, QualifiedExport, OpenCause);
			OpenedExports.AddUnique(QualifiedExport); 
		}
	}

	if(OpenedExports.Num() > 0)
	{
		WorkspaceView->SelectExport(OpenedExports.Last());
	}
}

void FWorkspaceEditor::OpenObjects(TConstArrayView<UObject*> InObjects, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride)
{
	const FDocumentTracker::EOpenDocumentCause OpenCause = OpenCauseOverride.Get(InObjects.Num() == 1 ? FDocumentTracker::EOpenDocumentCause::NavigatingCurrentDocument : FDocumentTracker::EOpenDocumentCause::OpenNewDocument);

	for(UObject* Object : InObjects)
	{
		const FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
		const FDocumentSubObjectArgs* DocumentSubObjectArgs = WorkspaceEditorModule.FindDocumentSubObjectType(Object);
		UObject* OriginalObject = Object;
		if(DocumentSubObjectArgs && DocumentSubObjectArgs->OnGetDocumentForSubObject.IsBound())
		{
			Object = DocumentSubObjectArgs->OnGetDocumentForSubObject.Execute(Object);
		}

		if(Object)
		{
			TSharedPtr<SDockTab> DocumentTab = OpenDocument(Object, OpenCause);
			if(DocumentSubObjectArgs && DocumentTab.IsValid())
			{
				TSharedRef<SWorkspaceTabWrapper> WorkspaceTabWrapper = StaticCastSharedRef<SWorkspaceTabWrapper>(DocumentTab->GetContent());
				TSharedRef<SWidget> TabContentWidget = WorkspaceTabWrapper->GetContent();

				// If this is a built-in graph editor widget, supply the inner SGraphEditor
				if(TabContentWidget->GetType() == TEXT("SGraphDocument"))
				{
					TabContentWidget = StaticCastSharedRef<SGraphDocument>(TabContentWidget)->GraphEditor.ToSharedRef();
					check(TabContentWidget->GetType() == TEXT("SGraphEditor"));
				}

				DocumentSubObjectArgs->OnPostDocumentOpenedForSubObject.ExecuteIfBound(FWorkspaceEditorContext(SharedThis(this), { FWorkspaceOutlinerItemExport(), Object }), TabContentWidget, OriginalObject);
			}
		}
	}
}

void FWorkspaceEditor::GetOpenedAssetsOfClass(TSubclassOf<UObject> InClass, TArray<UObject*>& OutAssets) const
{
	FWorkspaceAssetRegistryExports Exports;
	FWorkspaceEditorModule::GetExportedAssetsForWorkspace(FAssetData(Workspace), Exports);
	for(const FWorkspaceAssetRegistryExportEntry& Entry : Exports.Assets)
	{
		UObject* LoadedObject = Entry.Asset.ResolveObject();
		if(LoadedObject && LoadedObject->GetClass()->IsChildOf(InClass))
		{
			OutAssets.Add(LoadedObject);
		}
	}
}

void FWorkspaceEditor::GetAssets(TArray<FAssetData>& OutAssets, bool bIncludeAssetReferences /*= false */) const
{
	if (bIncludeAssetReferences)
	{
		WorkspaceView->GetHierarchyAssetData(OutAssets);
	}
	else
	{
		FWorkspaceAssetRegistryExports Exports;
		FWorkspaceEditorModule::GetExportedAssetsForWorkspace(FAssetData(Workspace), Exports);
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		for(const FWorkspaceAssetRegistryExportEntry& Entry : Exports.Assets)
		{
			OutAssets.Add(AssetRegistry.GetAssetByObjectPath(Entry.Asset));
		}
	}
}

void FWorkspaceEditor::CloseObjects(TConstArrayView<UObject*> InObjects)
{
	if(InObjects.Num() > 0)
	{
		for(const UObject* Object : InObjects)
		{
			CloseDocumentTab(Object);
		}
	}
}

void FWorkspaceEditor::SetDetailsObjects(const TArray<UObject*>& InObjects)
{
 	if(DetailsView.IsValid())
	{
		DetailsView->SetObjects(InObjects);
	}
}

void FWorkspaceEditor::RefreshDetails()
{
	if (DetailsView.IsValid())
	{
		DetailsView->RequestForceRefresh();
	}
}

UWorkspaceSchema* FWorkspaceEditor::GetSchema() const
{
	return Workspace.Get() != nullptr ? Workspace->GetSchema() : nullptr;
}

bool FWorkspaceEditor::GetOutlinerSelection(TArray<FWorkspaceOutlinerItemExport>& OutExports) const
{
	OutExports = LastSelectedExports;
	return LastSelectedExports.Num() > 0;
}

IWorkspaceEditor::FOnOutlinerSelectionChanged& FWorkspaceEditor::OnOutlinerSelectionChanged()
{
	return OnOutlinerSelectionChangedDelegate;
}

void FWorkspaceEditor::SetGlobalSelection(FGlobalSelectionId SelectionId, FOnClearGlobalSelection OnClearSelectionDelegate)
{
	if(SelectionScopeDepth == 0 || bSelectionScopeCleared == false)
	{
		TArray<TPair<FGlobalSelectionId, FOnClearGlobalSelection>> SelectionsCopy = GlobalSelections;
		GlobalSelections.Empty();
		
		for (TPair<FGlobalSelectionId, FOnClearGlobalSelection>& Selection : SelectionsCopy)
		{
			// Only execute if widget is still valid, and it is not the same as the previous call 
			if (Selection.Key.IsValid() && SelectionId != Selection.Key)
			{
				Selection.Value.ExecuteIfBound();
			}
		}
		
		bSelectionScopeCleared = true;
	}

	GlobalSelections.Add({ SelectionId, OnClearSelectionDelegate});
}

void FWorkspaceEditor::SetFocusedDocument(const UE::Workspace::FWorkspaceDocument& Document)
{
	if (bSettingFocusedDocument)
	{
		return;
	}	
	TGuardValue<bool> SetValueGuard(bSettingFocusedDocument, true);
	
	FAssetEditorModeManager* ModeManager = static_cast<FAssetEditorModeManager*>(&GetEditorModeManager());	
	ModeManager->GetSelectedObjects()->DeselectAll();

	UObject* DocumentObject = Document.GetObject();

	if (DocumentObject != nullptr)
	{
		ModeManager->GetSelectedObjects()->Select(DocumentObject);

		FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
		if (const FObjectDocumentArgs* Args = WorkspaceEditorModule.FindObjectDocumentType(Document))
		{
			const FEditorModeID NewEditorMode = Args->DocumentEditorMode;
			if (NewEditorMode != NAME_None)
			{
				if(!ModeManager->IsModeActive(NewEditorMode))
				{
					ModeManager->ActivateMode(NewEditorMode);
				}
			}
			else
			{
				ModeManager->DeactivateAllModes();
			}
		}

		LastDocument = Document;
	}
	else
	{
		ModeManager->DeactivateAllModes();
		LastDocument.Reset();
	}

	if (!IsViewportPinned())
	{
		ReinitViewportScene();
	}
	OnFocusedDocumentChangedDelegate.Broadcast(Document);
}

const FWorkspaceDocument& FWorkspaceEditor::GetFocusedDocumentOfClass(const TObjectPtr<UClass> AssetClass) const
{
	if (LastDocument.IsSet())
	{
		if (LastDocument.GetValue().GetObject() && LastDocument.GetValue().GetObject()->IsA(AssetClass))
		{
			return LastDocument.GetValue();
		}
	}

	static FWorkspaceDocument EmptyDocument;
	return EmptyDocument;
}

void FWorkspaceEditor::HandleOutlinerSelectionChanged(TConstArrayView<FWorkspaceOutlinerItemExport> InExports)
{
	LastSelectedExports = InExports;
}

FOnFocusedDocumentChanged& FWorkspaceEditor::OnFocusedDocumentChanged()
{
	return OnFocusedDocumentChangedDelegate;
}

TSharedPtr<IDetailsView> FWorkspaceEditor::GetDetailsView()
{
	return DetailsView;
}

UObject* FWorkspaceEditor::GetWorkspaceAsset() const
{
	return Workspace;
}

FString FWorkspaceEditor::GetPackageName() const
{
	return Workspace->GetPackage()->GetName();
}

const FWorkspaceDocument& FWorkspaceEditor::GetFocusedWorkspaceDocument() const
{
	if (LastDocument.IsSet())
	{
		return LastDocument.GetValue();
	}

	static FWorkspaceDocument EmptyDocument;
	return EmptyDocument;
}
	
void FWorkspaceEditor::ReinitViewportScene()
{
	FAdvancedPreviewScene* PreviewScene = static_cast<FAdvancedPreviewScene*>(ViewportClient->GetPreviewScene());
	if (ActiveViewportController)
	{
		ActiveViewportController->OnExit(PreviewScene);
		ActiveViewportController = nullptr;
	}

	const FSoftObjectPath AssetPath = ViewportPinnedAssetPath.IsValid()
		? ViewportPinnedAssetPath
		: GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();

	if (AssetPath.IsValid())
	{
		if (UObject* AssetObject = AssetPath.TryLoad())
		{
			FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
			ActiveViewportController = WorkspaceEditorModule.CreateViewportController(AssetObject->GetClass());

			if (ActiveViewportController)
			{
				IWorkspaceViewportController::FViewportContext ViewportContext;
				ViewportContext.PreviewScene = PreviewScene;
				ViewportContext.OutlinerObject = AssetObject;
				ViewportContext.SceneDescription = SceneDescription;
				
				ActiveViewportController->OnEnter(ViewportContext);
			}
		}
	}
}
	
void FWorkspaceEditor::BindCommands()
{
	const FWorkspaceAssetEditorCommands& Commands = FWorkspaceAssetEditorCommands::Get();

	ToolkitCommands->MapAction
	( 
		Commands.NavigateBackward,
		FExecuteAction::CreateRaw(this, &FWorkspaceEditor::NavigateBack)
	);
	
	ToolkitCommands->MapAction
	( 
		Commands.NavigateForward,
		FExecuteAction::CreateRaw(this, &FWorkspaceEditor::NavigateForward)
	);

	ToolkitCommands->MapAction
	(
		Commands.SaveAssetEntries,
		FExecuteAction::CreateRaw(this, &FWorkspaceEditor::SaveAssetEntries),
		FCanExecuteAction::CreateRaw(this, &FWorkspaceEditor::AreAssetEntriesModified)
	);
}

void FWorkspaceEditor::ExtendMenu() const
{
	
}

void FWorkspaceEditor::ExtendToolbar() const
{
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(GetToolMenuToolbarName()))
	{
		// Keep as other systems hook off of this section
		Menu->AddSection("WorkspaceOperations");
	}
}

void FWorkspaceEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FBaseAssetToolkit::RegisterTabSpawners(InTabManager);
	
	EditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_WorkspaceEditor", "Workspace Editor"));    

	InTabManager->RegisterTabSpawner(WorkspaceTabs::WorkspaceView, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args) -> TSharedRef<SDockTab>
	{
		check(Args.GetTabId() == WorkspaceTabs::WorkspaceView);
		
		return SNew(SDockTab)
			.Label(LOCTEXT("WorkspaceTabLabel", "Outliner"))
			[
				WorkspaceView.ToSharedRef()
			];
	}))
	.SetDisplayName(LOCTEXT("WorkspaceTabLabel", "Outliner"))
	.SetIcon(FSlateIcon("EditorStyle", "LevelEditor.Tabs.Outliner"))
	.SetTooltipText(LOCTEXT("WorkspaceTabToolTip", "Shows the workspace outliner tab."));

	DocumentManager->SetTabManager(InTabManager);

	Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	if (WorkspaceEditorModule.OnRegisterTabsForEditor().IsBound())
	{
		WorkspaceEditorModule.OnRegisterTabsForEditor().Broadcast(TabFactories, InTabManager, StaticCastSharedPtr<IWorkspaceEditor>(this->AsShared().ToSharedPtr()));
	}

	if (!GetSchema()->SupportsViewport())
	{
		InTabManager->UnregisterTabSpawner(ViewportTabID);
	}
}

void FWorkspaceEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FBaseAssetToolkit::UnregisterTabSpawners(InTabManager);
}

FName FWorkspaceEditor::GetToolkitFName() const
{
	return IWorkspaceEditor::GetWorkspaceEditorToolkitName();
}

FText FWorkspaceEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "WorkspaceEditor");
}

FString FWorkspaceEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "WorkspaceEditor ").ToString();
}

FLinearColor FWorkspaceEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FWorkspaceEditor::InitToolMenuContext(FToolMenuContext& InMenuContext)
{
	UAssetEditorToolkitMenuContext* ToolkitMenuContext = NewObject<UAssetEditorToolkitMenuContext>();
	ToolkitMenuContext->Toolkit = AsShared();
	InMenuContext.AddObject(ToolkitMenuContext);

	IWorkspaceEditor::InitToolMenuContext(InMenuContext);

	Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	WorkspaceEditorModule.OnExtendToolMenuContext().Broadcast(StaticCastSharedRef<UE::Workspace::IWorkspaceEditor>(AsShared()), InMenuContext);
}

void FWorkspaceEditor::SaveAsset_Execute()
{
	if (GetSchema()->CanSaveWorkspace())
	{
		// If asset is a default 'Untitled' workspace, redirect to the 'save as' flow
		const FString AssetPath = Workspace->GetOutermost()->GetPathName();
		if(AssetPath.StartsWith(TEXT("/Temp/Untitled")))
		{
			// Ensure we do not also 'save as' other externally linked assets at this point
			{
				TGuardValue<bool> SavingTransientWorkspace(bSavingTransientWorkspace, true);
				SaveAssetAs_Execute();
			}
		}
		else
		{
			TGuardValue<bool> SavingWorkspace(bSavingWorkspace, true);
			FBaseAssetToolkit::SaveAsset_Execute();
		}
	}
	else
	{
		// Saves the assets returned by our GetSaveableObjects implementation
		FBaseAssetToolkit::SaveAsset_Execute();
	}
}

void FWorkspaceEditor::CloseDocumentTab(const UObject* DocumentID) const
{
	UWorkspaceState* State = Workspace->GetState();

	// Close all exports that reference the document's object
	FSoftObjectPath ObjectPath(DocumentID);
	for (const TInstancedStruct<FWorkspaceDocumentState>& InstancedStruct : State->DocumentStates)
	{
		const FWorkspaceDocumentState& DocumentState = InstancedStruct.Get<FWorkspaceDocumentState>();
		if (DocumentState.Object == ObjectPath)
		{
			const TSharedRef<FTabPayload_WorkspaceDocument> Payload = FTabPayload_WorkspaceDocument::Make(DocumentID, DocumentState.Export);
			DocumentManager->CloseTab(Payload);
		}
	}

	// Clean up document states
	State->DocumentStates.RemoveAll([&ObjectPath](const TInstancedStruct<FWorkspaceDocumentState>& InDocumentState)
	{
		return InDocumentState.Get<FWorkspaceDocumentState>().Object == ObjectPath;
	});
}

bool FWorkspaceEditor::InEditingMode() const 
{
	return true;
}

void FWorkspaceEditor::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		if (bSavingWorkspace || bSavingTransientWorkspace)
		{
			FBaseAssetToolkit::GetSaveableObjects(OutObjects);
		}
    	
		if(bSavingAssetEntries)
		{
			for(const UWorkspaceAssetEntry* Entry : Workspace->AssetEntries)
			{
				if(UObject* Asset = Entry->Asset.Get())
				{
					// Add object referenced by workspace
					OutObjects.Add(Asset);
				}
			}
		}
	}
	else
	{
		if (UObject* FocusedAsset = GetFocusedAsset())
		{
			OutObjects.Add(FocusedAsset);
		}
	}
}

bool FWorkspaceEditor::CanSaveAsset() const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		bool bDirtyState = false;
		for (UObject* EditingObject : GetEditingObjects())
		{
			const UPackage* Package = EditingObject->GetPackage();
			if (Package->IsDirty() || Package->GetExternalPackages().ContainsByPredicate([](const UPackage* ExternalPackage)  { return ExternalPackage && ExternalPackage->IsDirty(); }))
			{
				bDirtyState = true;
				break;
			}
		}
    
		return bDirtyState;
	}
	else
	{
		if (const UObject* RootAsset = GetFocusedAsset())
		{
			const UPackage* RootAssetPackage = RootAsset->GetPackage();
			return RootAssetPackage->IsDirty();
		}

		return false;
	}
}

FText FWorkspaceEditor::GetTabSuffix() const
{
	return CanSaveAsset() ? LOCTEXT("TabSuffixAsterix", "*") : FText::GetEmpty();
}

FText FWorkspaceEditor::GetToolkitName() const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		UObject* const* WorkspaceObject = GetEditingObjects().FindByPredicate([](const UObject* Object) { return Object && Object->IsA<UWorkspace>(); });
		check (WorkspaceObject != nullptr);
    	
		return GetLabelForObject(*WorkspaceObject);
	}
	
	if (const UObject* RootAsset = Workspace->GetRootAsset())
	{
		return GetLabelForObject(RootAsset);
	}

	if (const UObject* FocusedAsset = GetFocusedAsset())
	{
		return GetLabelForObject(FocusedAsset);
	}

	return LOCTEXT("Workspace", "Workspace");
}

FText FWorkspaceEditor::GetToolkitToolTipText() const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		UObject* const* WorkspaceObject = GetEditingObjects().FindByPredicate([](const UObject* Object) { return Object && Object->IsA<UWorkspace>(); });
		check (WorkspaceObject != nullptr);
		
		FText FocusedAssetText = FText::FromName(NAME_None);
	
		const FWorkspaceDocument& Document = GetFocusedWorkspaceDocument();
		if (const UObject* FocusedObject = Document.GetObject())
		{
			 FocusedAssetText = GetLabelForObject(FocusedObject);
		}
		
		return FText::Format(LOCTEXT("ToolkitTooltipFormat", "{0} ({1})"), GetToolTipTextForObject(*WorkspaceObject), FocusedAssetText);
	}
	
	if (const UObject* RootAsset = Workspace->GetRootAsset())
	{
		return GetLabelForObject(RootAsset);
	}
	
	if (const UObject* FocusedAsset = GetFocusedAsset())
	{
		return GetLabelForObject(FocusedAsset);
	}

	return LOCTEXT("Workspace", "Workspace");
}

const FSlateBrush* FWorkspaceEditor::GetDefaultTabIcon() const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		return FBaseAssetToolkit::GetDefaultTabIcon();
	}
	
	if (const UObject* RootAsset = Workspace->GetRootAsset())
    {
		return FSlateIconFinder::FindIconBrushForClass(RootAsset->GetClass());
	}
	
	if (const UObject* FocusedAsset = GetFocusedAsset())
	{
		return FSlateIconFinder::FindIconBrushForClass(FocusedAsset->GetClass());
	}

	return FAssetEditorToolkit::GetDefaultTabIcon();
}

FLinearColor FWorkspaceEditor::GetDefaultTabColor() const
{
	if (GetSchema()->CanSaveWorkspace())
	{
		return FBaseAssetToolkit::GetDefaultTabColor();
	}

	static const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	
	if (const UObject* RootAsset = Workspace->GetRootAsset())
	{
		const UClass* RootAssetClass = RootAsset->GetClass();

		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(RootAssetClass);
		if (AssetTypeActions.IsValid())
		{
			return AssetTypeActions.Pin()->GetTypeColor();
		}
	}

	if (const UObject* FocusedAsset = GetFocusedAsset())
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(FocusedAsset->GetClass());
		if (AssetTypeActions.IsValid())
		{
			return AssetTypeActions.Pin()->GetTypeColor();
		}
	}
	
    return FLinearColor::White;
}

void FWorkspaceEditor::RecordDocumentState(const TInstancedStruct<FWorkspaceDocumentState>& InState) const
{
	UWorkspaceState* State = Workspace->GetState();
	State->DocumentStates.AddUnique(InState);
}

void FWorkspaceEditor::NavigateBack() const
{
	const TSharedRef<FTabPayload_WorkspaceDocument> Payload = FTabPayload_WorkspaceDocument::Make(nullptr);
	TSharedPtr<SDockTab> OpenedTab = DocumentManager->OpenDocument(Payload, FDocumentTracker::NavigateBackwards);
}

void FWorkspaceEditor::NavigateForward() const
{
	const TSharedRef<FTabPayload_WorkspaceDocument> Payload = FTabPayload_WorkspaceDocument::Make(nullptr);
	TSharedPtr<SDockTab> OpenedTab = DocumentManager->OpenDocument(Payload, FDocumentTracker::NavigateForwards);
}

void FWorkspaceEditor::SaveAssetEntries()
{	
	TGuardValue<bool> SavingAssetEntries(bSavingAssetEntries, true);
	FBaseAssetToolkit::SaveAsset_Execute();
}

bool FWorkspaceEditor::AreAssetEntriesModified() const
{
	for(const UWorkspaceAssetEntry* Entry : Workspace->AssetEntries)
	{
		if (Entry != nullptr)
		{
			if (Entry->Asset.IsValid())
			{
				if (UObject* Asset = Entry->Asset.Get())
				{
					const UPackage* Package = Asset->GetOutermost();
					if (Package->IsDirty() || Package->GetExternalPackages().ContainsByPredicate([](const UPackage* ExternalPackage)  { return ExternalPackage && ExternalPackage->IsDirty(); }))
					{
						return true;
					}
				}
			}
			
		}
	}
	
	return false;
}

bool FWorkspaceEditor::IsViewportPinned() const
{
	return ViewportPinnedAssetPath.IsValid();
}

#if WITH_ADDITIONAL_CRASH_CONTEXTS
void FWorkspaceEditor::OnProvideAdditionalCrashContext(FCrashContextExtendedWriter& CrashContextExtendedWriter) const
{
	thread_local TStringBuilder<1024> Builder;  
	{
		// Workspace asset
		Builder.Append(GetNameSafe(Workspace));
		Builder << TEXT('\n');

		// Workspace asset
		if (Workspace)
		{
			Builder << TEXT("Root Asset: ");
			const UObject* RootAsset = Workspace->GetRootAsset();
			Builder.Append(GetNameSafe(RootAsset));
			Builder << TEXT("\nLoaded Assets:\n");

			TArray<TObjectPtr<UObject>> LoadedWorkspaceAssets;
			Workspace->GetAssets(LoadedWorkspaceAssets);
			for (const TObjectPtr<UObject>& Asset : LoadedWorkspaceAssets)
			{
				Builder << TEXT("\t+ ");
				Builder.Append(GetNameSafe(RootAsset));
				Builder << TEXT('\n');
			}
		}

		const FWorkspaceDocument& CurrentDocument = GetFocusedWorkspaceDocument();
		if (CurrentDocument.Export.GetIdentifier() != NAME_None)
		{
			Builder << TEXT("Focussed Document:\n");
			Builder << TEXT("\tExport Path: ");
			Builder.Append(CurrentDocument.Export.GetFullPath());
			Builder << TEXT("\tObject: ");
			Builder.Append(CurrentDocument.Object ? CurrentDocument.Object->GetPathName() : TEXT("None"));
			Builder << TEXT('\n');
		}
	}
	
	CrashContextExtendedWriter.AddString(*GetEditorName().ToString(), Builder.ToString());
}
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

void FWorkspaceEditor::FindInContentBrowser_Execute()
{
	if (IsValid(Workspace))
	{
		if (Workspace->GetSchema()->CanSaveWorkspace())
		{
			GEditor->SyncBrowserToObject(Workspace);
		}
		else
		{
			if (UObject* FocusedAsset = GetFocusedAsset())
			{
				GEditor->SyncBrowserToObject(FocusedAsset);
			}
		}
	}
}

bool FWorkspaceEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if (!GetSchema()->CanSaveWorkspace())
	{
		return true;
	}
	
	TGuardValue<bool> ClosingDown(bClosingDown, true);

	auto RequiresSave = [this]()
		{
			const UPackage* Package = Workspace->GetOutermost();
			// Ask the user to save a transient workspace containing more than 1 asset
			return Package->GetPathName().StartsWith(TEXT("/Temp/Untitled")) && Workspace->AssetEntries.Num() > 1;
		};

	// Give the user opportunity to save temp workspaces
	if(RequiresSave() && !bSavingTransientWorkspace)
	{
		// Prompt whether to save or not, this can be skipped to become a never-ask-nor-save
		FSuppressableWarningDialog::FSetupInfo Info(
		LOCTEXT("SavingTransientWorkspaceAssetMessage", "Asset was opened in a temporary Workspace, do you want to save it?"),
				LOCTEXT("SavingTransientWorkspaceAssetTitle", "Save temporary Workspace"), "SaveTemporaryWorkspacesPrompt");

		Info.DialogMode = FSuppressableWarningDialog::EMode::PersistUserResponse;
		Info.ConfirmText =LOCTEXT("SavingTransientWorkspaceAssetYes", "Yes");
		Info.CancelText = LOCTEXT("SavingTransientWorkspaceAssetNo", "No");

		FSuppressableWarningDialog SaveWorkspace(Info);
		FSuppressableWarningDialog::EResult Result = SaveWorkspace.ShowModal();

		if(Result == FSuppressableWarningDialog::EResult::Confirm)
		{
			// Ensure we do not also 'save as' other externally linked assets at this point
			TGuardValue<bool> SaveWorkspaceOnly(bSavingTransientWorkspace, true);
			SaveAssetAs_Execute();
		}
	}

	return true;
}

void FWorkspaceEditor::OnClose()
{
	if (GetSchema()->CanSaveWorkspace())
	{
		SaveEditedObjectState();
	}
	
	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(nullptr);
		DetailsView.Reset();
	}

	for (TSharedPtr<class IToolkit> Toolkit : HostedToolkits)
	{
		if (Toolkit.IsValid())
		{
			UToolMenus::UnregisterOwner(Toolkit.Get());
		}
	}

	TabFactories.Clear();
	FBaseAssetToolkit::OnClose();
}

void FWorkspaceEditor::RegisterToolbar()
{
	IWorkspaceEditor::RegisterToolbar();
}

bool FWorkspaceEditor::ShouldReopenEditorForSavedAsset(const UObject* Asset) const
{
	return !bClosingDown;
}

void FWorkspaceEditor::RemoveEditingObject(UObject* Object)
{
	IWorkspaceEditor::RemoveEditingObject(Object);
	if (LastDocument.IsSet() && LastDocument->GetObject() == Object)
	{
		FWorkspaceDocument EmptyDocument;
		SetFocusedDocument(EmptyDocument);
	}
}

AssetEditorViewportFactoryFunction FWorkspaceEditor::GetViewportDelegate()
{
	SetSceneDescription(GetSchema()->CreateSceneDescription());
	
	return [this](FAssetEditorViewportConstructionArgs InArgs)
		{
			TSharedRef<UE::Workspace::SWorkspaceViewport> Viewport = SNew(UE::Workspace::SWorkspaceViewport)
				.AssetEditorToolkit(SharedThis(this))
				.ViewportClient(ViewportClient)
				.SceneDescription(SceneDescription)
				.bIsPinned_Lambda([this]()
				{
					return ViewportPinnedAssetPath.IsValid();
				})
				.OnPinnedClicked_Lambda([this]()
				{
					if (ViewportPinnedAssetPath.IsValid())
					{
						const bool bShouldReinitViewport = ViewportPinnedAssetPath != GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();
						ViewportPinnedAssetPath.Reset();

						if (bShouldReinitViewport)
						{
							ReinitViewportScene();
						}
					}
					else
					{
						ViewportPinnedAssetPath = GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();
					}
				})
				.PreviewAssetPath_Lambda([this]()
				{
					return ViewportPinnedAssetPath.IsValid()
						? ViewportPinnedAssetPath
						: GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();
				});
			
			return Viewport;
		};
}
	
TSharedPtr<FEditorViewportClient> FWorkspaceEditor::CreateEditorViewportClient() const
{
	TSharedPtr<FAdvancedPreviewScene> NewAdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
	
	StaticCastSharedPtr<FAssetEditorModeManager>(EditorModeManager)->SetPreviewScene(NewAdvancedPreviewScene.Get());

	TSharedPtr<FEditorViewportClient> EditorViewportClient = MakeShared<FWorkspaceAssetViewportClient>(EditorModeManager.Get(), NewAdvancedPreviewScene);
	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation(FVector(0.0f, 400.0f, 200.0f));
	EditorViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);
	EditorViewportClient->SetRealtime(true);
	EditorViewportClient->SetViewMode(VMI_Lit);
	EditorViewportClient->ToggleOrbitCamera(true);

	return EditorViewportClient;
}

void FWorkspaceEditor::SetSceneDescription(TObjectPtr<UWorkspaceViewportSceneDescription> InSceneDescription)
{
	SceneDescription = InSceneDescription;

	SceneDescription->GetOnConfigChanged().AddSP(this, &FWorkspaceEditor::ReinitViewportScene);
}
	
UWorkspaceViewportSceneDescription* FWorkspaceEditor::GetSceneDescription() const
{
	return SceneDescription;
}

UObject* FWorkspaceEditor::GetFocusedAsset() const
{
	return GetFocusedWorkspaceDocument().Export.GetFirstAssetOfType<UObject>();
}

} // namespace UE::Workspace

#undef LOCTEXT_NAMESPACE
