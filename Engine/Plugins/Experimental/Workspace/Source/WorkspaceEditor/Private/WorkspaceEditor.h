// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "WorkspaceEditorModeUILayer.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

struct FWorkspaceDocumentState;
struct FWorkspaceAssetRegistryExports;
class UWorkspace;
class FDocumentTracker;
class FDocumentTabFactory;
class FTabInfo;
class FTabManager;
class IToolkitHost;
class SWorkspaceTabWrapper;
class IStructureDetailsView;

namespace UE::Workspace
{
struct FGraphDocumentSummoner;
struct FWorkspaceTabSummoner;
struct FAssetDocumentSummoner;
class FWorkspaceEditorModule;
class SWorkspaceView;
class IWorkspaceViewportController;

namespace WorkspaceModes
{
	extern const FName WorkspaceEditor;
}

namespace WorkspaceTabs
{
	extern const FName Details;
	extern const FName WorkspaceView;
}

class FWorkspaceEditor : public IWorkspaceEditor, public FGCObject
{
public:
	FWorkspaceEditor(UWorkspaceAssetEditor* InOwningAssetEditor);
	virtual ~FWorkspaceEditor() override {}

private:
	friend class FWorkspaceEditorMode;
	friend class FWorkspaceEditorModule;
	friend class SGraphDocument;
	friend SWorkspaceTabWrapper;
	friend struct FGraphDocumentSummoner;
	friend struct FWorkspaceTabSummoner;
	friend struct FAssetDocumentSummoner;
	friend struct FWorkspaceEditorSelectionScope;

	// FBaseAssetToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void CreateWidgets() override;
	virtual void PostInitAssetEditor() override;
	virtual void InitToolMenuContext(FToolMenuContext& InMenuContext) override;
	virtual void SaveAsset_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual void OnClose() override;
	virtual void RegisterToolbar() override;
	virtual bool ShouldReopenEditorForSavedAsset(const UObject* Asset) const override;
	virtual void RemoveEditingObject(UObject* Object) override;
	virtual AssetEditorViewportFactoryFunction GetViewportDelegate() override;
	virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const override;
	
	// FAssetEditorToolkit interface
	virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	virtual bool CanSaveAsset() const override;
	virtual FText GetTabSuffix() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual const FSlateBrush* GetDefaultTabIcon() const override;
    virtual FLinearColor GetDefaultTabColor() const override;
	virtual void FindInContentBrowser_Execute() override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FWorkspaceEditor");
	}

	// IWorkspaceEditor interface
	virtual void OpenAssets(TConstArrayView<FAssetData> InAssets) override;
	virtual void OpenExports(TConstArrayView<FWorkspaceOutlinerItemExport> InExports, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride) override;
	virtual void OpenObjects(TConstArrayView<UObject*> InObjects, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride) override;
	virtual void GetOpenedAssetsOfClass(TSubclassOf<UObject> InClass, TArray<UObject*>& OutObjects) const override;
	virtual void GetAssets(TArray<FAssetData>& OutAssets, bool bIncludeAssetReferences = false) const override;
	virtual void CloseObjects(TConstArrayView<UObject*> InObjects) override;
	virtual void SetDetailsObjects(const TArray<UObject*>& InObjects) override;
	virtual void RefreshDetails() override;
	virtual UWorkspaceSchema* GetSchema() const override;
	virtual bool GetOutlinerSelection(TArray<FWorkspaceOutlinerItemExport>& OutExports) const override;
	virtual FOnOutlinerSelectionChanged& OnOutlinerSelectionChanged() override;
	virtual void SetGlobalSelection(FGlobalSelectionId SelectionId, FOnClearGlobalSelection OnClearSelectionDelegate) override;
	virtual const FWorkspaceDocument& GetFocusedDocumentOfClass(const TObjectPtr<UClass> AssetClass) const override;
	virtual FOnFocusedDocumentChanged& OnFocusedDocumentChanged() override;
	virtual TSharedPtr<IDetailsView> GetDetailsView() override;
	virtual UObject* GetWorkspaceAsset() const override;
	virtual FString GetPackageName() const override;
	virtual const FWorkspaceDocument& GetFocusedWorkspaceDocument() const override;
	virtual void SetSceneDescription(TObjectPtr<UWorkspaceViewportSceneDescription> InSceneDescription) override;
	virtual UWorkspaceViewportSceneDescription* GetSceneDescription() const override;
	virtual UObject* GetFocusedAsset() const override;
	
	void ReinitViewportScene();

	void HandleOutlinerSelectionChanged(TConstArrayView<FWorkspaceOutlinerItemExport> InExports);

	void BindCommands();

	void ExtendMenu() const;

	void ExtendToolbar() const;

	void CloseDocumentTab(const UObject* DocumentID) const;

	bool InEditingMode() const;

	void RestoreEditedObjectState();

	void SaveEditedObjectState() const;

	TSharedPtr<SDockTab> OpenDocument(const UObject* InForObject, FDocumentTracker::EOpenDocumentCause InCause);
	TSharedPtr<SDockTab> OpenDocument(const UObject* InForObject, const FWorkspaceOutlinerItemExport& InExport, FDocumentTracker::EOpenDocumentCause InCause);

	void RecordDocumentState(const TInstancedStruct<FWorkspaceDocumentState>& InState) const;

	void SetFocusedDocument(const UE::Workspace::FWorkspaceDocument& Document);

	void NavigateBack() const;
	void NavigateForward() const;

	void SaveAssetEntries();
	bool AreAssetEntriesModified() const;

	bool IsViewportPinned() const;
		
#if WITH_ADDITIONAL_CRASH_CONTEXTS
	void OnProvideAdditionalCrashContext(FCrashContextExtendedWriter& CrashContextExtendedWriter) const;
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

	/** The asset being edited */
	TObjectPtr<UWorkspace> Workspace = nullptr;

	// Document tracker
	TSharedPtr<FDocumentTracker> DocumentManager;

	TSharedPtr<SWorkspaceView> WorkspaceView;

	TMap<FName, TSharedPtr<FWorkspaceEditorModeUILayer>> ModeUILayers;
	TArray<TSharedPtr<class IToolkit>> HostedToolkits;

	/** Tabs to be registered into the Workspace */
	FWorkflowAllowedTabSet TabFactories;

	bool bSavingTransientWorkspace = false;
	bool bSavingWorkspace = false;
	bool bSavingAssetEntries = false;
	bool bClosingDown = false;
	bool bSettingFocusedDocument = false;

	TArray<TPair<FGlobalSelectionId, FOnClearGlobalSelection>> GlobalSelections;
	FOnOutlinerSelectionChanged OnOutlinerSelectionChangedDelegate;
	bool bSelectionScopeCleared = true;
	int32 SelectionScopeDepth = 0;
	
	TArray<FWorkspaceOutlinerItemExport> LastSelectedExports;
	FOnFocusedDocumentChanged OnFocusedDocumentChangedDelegate;
	TSharedPtr<FWorkspaceItem> EditorMenuCategory;

	TOptional<FWorkspaceDocument> LastDocument;

	TUniquePtr<IWorkspaceViewportController> ActiveViewportController;
	TObjectPtr<UWorkspaceViewportSceneDescription> SceneDescription;
	FSoftObjectPath ViewportPinnedAssetPath;
};

}
