// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceEditorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "WorkspaceEditorCommands"

namespace UE::Workspace
{
	FWorkspaceAssetEditorCommands::FWorkspaceAssetEditorCommands()
		: TCommands<FWorkspaceAssetEditorCommands>(
				"WorkspaceAssetEditor",
				NSLOCTEXT("Contexts", "WorkspaceAssetEditor", "Workspace Editor"),
				NAME_None,
				FAppStyle::GetAppStyleSetName()
			)
	{
	}

	void FWorkspaceAssetEditorCommands::RegisterCommands()
	{
		UI_COMMAND(NavigateBackward, "Navigate Backward", "Moves backwards to previous location",
			EUserInterfaceActionType::Button, FInputChord(EKeys::ThumbMouseButton), FInputChord(EKeys::Left, EModifierKey::Alt));
		UI_COMMAND(NavigateForward, "Navigate Forward", "Moves forwards to previous location",
			EUserInterfaceActionType::Button, FInputChord(EKeys::ThumbMouseButton2), FInputChord(EKeys::Right, EModifierKey::Alt));
		UI_COMMAND(SaveAssetEntries, "Save Assets", "Saves all loaded and modified assets in the workspace", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Open, "Open", "Open items", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter))
	}

}  // namespace UE::Workspace

#undef LOCTEXT_NAMESPACE

