// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

namespace UE::Workspace
{
	class FWorkspaceAssetEditorCommands : public TCommands<FWorkspaceAssetEditorCommands>
	{
	public:
		FWorkspaceAssetEditorCommands();
		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> NavigateBackward;
		TSharedPtr<FUICommandInfo> NavigateForward;
		TSharedPtr<FUICommandInfo> SaveAssetEntries;
		TSharedPtr<FUICommandInfo> Open;
	};

}  // namespace UE::Workspace

