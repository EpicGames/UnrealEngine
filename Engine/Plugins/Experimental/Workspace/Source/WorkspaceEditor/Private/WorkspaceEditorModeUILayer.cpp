// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceEditorModeUILayer.h"
#include "Toolkits/IToolkit.h"
#include "ToolMenus.h"

FWorkspaceEditorModeUILayer::FWorkspaceEditorModeUILayer(const IToolkitHost* InToolkitHost) : FAssetEditorModeUILayer(InToolkitHost)
{
}

void FWorkspaceEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();

		OnToolkitHostReadyForUI.Execute();

		UToolMenu* SecondaryModeToolbar = UToolMenus::Get()->ExtendMenu(GetSecondaryModeToolbarName());
		OnRegisterSecondaryModeToolbarExtension.ExecuteIfBound(SecondaryModeToolbar);
	}
}

void FWorkspaceEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{	
	FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
}

void FWorkspaceEditorModeUILayer::SetModeMenuCategory(const TSharedPtr<FWorkspaceItem>& MenuCategoryIn)
{
	MenuCategory = MenuCategoryIn;
}

TSharedPtr<FWorkspaceItem> FWorkspaceEditorModeUILayer::GetModeMenuCategory() const
{
	check(MenuCategory);
	return MenuCategory;
}