// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorModeUILayer.h"

class FWorkspaceEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	FWorkspaceEditorModeUILayer(const IToolkitHost* InToolkitHost);
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;
	
	void SetModeMenuCategory(const TSharedPtr<FWorkspaceItem>& MenuCategoryIn);
protected:
	TSharedPtr<FWorkspaceItem> MenuCategory;
};