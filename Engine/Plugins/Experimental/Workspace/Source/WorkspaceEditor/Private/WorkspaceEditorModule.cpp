// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceEditorModule.h"

#include "EdGraphUtilities.h"
#include "ExternalPackageHelper.h"
#include "WorkspaceDocumentState.h"
#include "GraphDocumentState.h"
#include "SGraphDocument.h"
#include "SWorkspacePicker.h"
#include "Workspace.h"
#include "WorkspaceAssetEditor.h"
#include "WorkspaceEditor.h"
#include "WorkspaceEditorCommands.h"
#include "WorkspaceFactory.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "WorkspaceAssetReferenceItemDetails.h"
#include "WorkspaceGroupItemDetails.h"
#include "WorkspaceSchema.h"

#define LOCTEXT_NAMESPACE "WorkspaceEditorModule"

namespace UE::Workspace
{

TMap<FOutlinerItemDetailsId, TSharedPtr<IWorkspaceOutlinerItemDetails>> FWorkspaceEditorModule::OutlinerItemDetails;

FWorkspaceDocument::FWorkspaceDocument(const FWorkspaceOutlinerItemExport& InExport, UObject* InObject) : Export(InExport), Object(InObject)
{
	check(InObject);
	FWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<FWorkspaceEditorModule>("WorkspaceEditor");
	const FObjectDocumentArgs* FoundArgs = WorkspaceEditorModule.ObjectDocumentArgs.Find(InObject->GetClass()->GetClassPathName());
	while(FoundArgs && FoundArgs->OnRedirectWorkspaceContext.IsBound())
	{
		InObject = FoundArgs->OnRedirectWorkspaceContext.Execute(*this);
		FoundArgs = WorkspaceEditorModule.ObjectDocumentArgs.Find(InObject->GetClass()->GetClassPathName());
	}

	Object = InObject;
}

FWorkspaceEditorContext::FWorkspaceEditorContext(const TSharedRef<IWorkspaceEditor>& InWorkspaceEditor, UObject* InObject, const FWorkspaceOutlinerItemExport& InExport) : FWorkspaceEditorContext(InWorkspaceEditor, { InExport, InObject })
{
}

FWorkspaceEditorContext::FWorkspaceEditorContext(const TSharedRef<IWorkspaceEditor>& InWorkspaceEditor, const FWorkspaceDocument& InDocument) : WorkspaceEditor(InWorkspaceEditor), Document(InDocument)
{
}

void FWorkspaceEditorModule::StartupModule()
{
	FWorkspaceAssetEditorCommands::Register();
	
	TSharedPtr<FWorkspaceAssetReferenceOutlinerItemDetails> ReferenceItemDetails = MakeShareable<FWorkspaceAssetReferenceOutlinerItemDetails>(new FWorkspaceAssetReferenceOutlinerItemDetails());
	RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FWorkspaceOutlinerAssetReferenceItemData::StaticStruct()->GetFName()), StaticCastSharedPtr<UE::Workspace::IWorkspaceOutlinerItemDetails>(ReferenceItemDetails));

	TSharedPtr<FWorkspaceGroupOutlinerItemDetails> GroupItemDetails = MakeShareable<FWorkspaceGroupOutlinerItemDetails>(new FWorkspaceGroupOutlinerItemDetails());
	RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FWorkspaceOutlinerGroupItemData::StaticStruct()->GetFName()), StaticCastSharedPtr<UE::Workspace::IWorkspaceOutlinerItemDetails>(GroupItemDetails));
}

void FWorkspaceEditorModule::RegisterObjectDocumentType(const FTopLevelAssetPath& InClassPath, const FObjectDocumentArgs& InParams)
{
	ensure(InParams.SpawnLocation != NAME_None);

	DocumentAreaMap.FindOrAdd(InParams.SpawnLocation).Add(InClassPath);
	ObjectDocumentArgs.Add(InClassPath, InParams);
}

void FWorkspaceEditorModule::UnregisterObjectDocumentType(const FTopLevelAssetPath& InClassPath)
{
	if(const FObjectDocumentArgs* ExistingType = ObjectDocumentArgs.Find(InClassPath))
	{
		DocumentAreaMap.FindChecked(ExistingType->SpawnLocation).Remove(InClassPath);
	}
	ObjectDocumentArgs.Remove(InClassPath);
}

void FWorkspaceEditorModule::RegisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath, const FDocumentSubObjectArgs& InParams)
{
	DocumentSubObjectArgs.Add(InClassPath, InParams);
}

void FWorkspaceEditorModule::UnregisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath)
{
	DocumentSubObjectArgs.Remove(InClassPath);
}

FObjectDocumentArgs FWorkspaceEditorModule::CreateGraphDocumentArgs(const FGraphDocumentWidgetArgs& InArgs)
{
	FObjectDocumentArgs Args;
	Args.OnMakeDocumentWidget = FOnMakeDocumentWidget::CreateLambda([InArgs](const FWorkspaceEditorContext& InContext)
	{
		TWeakPtr<FWorkspaceEditor> WeakWorkspaceEditor = StaticCastSharedRef<FWorkspaceEditor>(InContext.WorkspaceEditor);
		TSharedRef<SWidget> Widget = SNew(SGraphDocument, StaticCastSharedRef<FWorkspaceEditor>(InContext.WorkspaceEditor), InContext.Document)
			.OnCreateActionMenu_Lambda([OnCreateActionMenu = InArgs.OnCreateActionMenu](const FWorkspaceEditorContext& InContext, UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bInAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
			{
				if(OnCreateActionMenu.IsBound())
				{
					return OnCreateActionMenu.Execute(InContext, InGraph, InNodePosition, InDraggedPins, bInAutoExpand, InOnMenuClosed);
				}
				return FActionMenuContent();
			})
			.OnNodeTextCommitted(InArgs.OnNodeTextCommitted)
			.OnGraphSelectionChanged(InArgs.OnGraphSelectionChanged)
			.OnCanDeleteSelectedNodes(InArgs.OnCanDeleteSelectedNodes)
			.OnDeleteSelectedNodes(InArgs.OnDeleteSelectedNodes)
			.OnCanCutSelectedNodes(InArgs.OnCanCutSelectedNodes)
			.OnCutSelectedNodes(InArgs.OnCutSelectedNodes)
			.OnCanCopySelectedNodes(InArgs.OnCanCopySelectedNodes)
			.OnCopySelectedNodes(InArgs.OnCopySelectedNodes)
			.OnCanPasteNodes(InArgs.OnCanPasteNodes)
			.OnPasteNodes(InArgs.OnPasteNodes)
			.OnCanDuplicateSelectedNodes(InArgs.OnCanDuplicateSelectedNodes)
			.OnDuplicateSelectedNodes(InArgs.OnDuplicateSelectedNodes)
			.OnNavigateHistoryForward_Lambda([WeakWorkspaceEditor](){ if(const TSharedPtr<FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin()) SharedWorkspaceEditor->NavigateForward(); })
			.OnNavigateHistoryBack_Lambda([WeakWorkspaceEditor](){ if(const TSharedPtr<FWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin()) SharedWorkspaceEditor->NavigateBack(); })
			.OnNodeDoubleClicked(InArgs.OnNodeDoubleClicked)
			.OnCanOpenInNewTab(InArgs.OnCanOpenInNewTab)
			.OnOpenInNewTab(InArgs.OnOpenInNewTab)
			.OnCreateComment(InArgs.OnCreateComment);

		// Let listeners know that the document widget has been created 
		InArgs.OnGraphDocumentCreated.ExecuteIfBound(InContext, Widget);

		return Widget;
	});
	Args.OnGetTabIcon = FOnGetTabIcon::CreateLambda([](const FWorkspaceEditorContext& InContext)
	{
		return FAppStyle::Get().GetBrush(TEXT("GraphEditor.EventGraph_16x"));
	});
	Args.OnGetTabName = FOnGetTabName::CreateLambda([](const FWorkspaceEditorContext& InContext)
	{
		if(UEdGraph* Graph = InContext.Document.GetTypedObject<UEdGraph>())
		{
			if (const UEdGraphSchema* Schema = Graph->GetSchema())
			{
				FGraphDisplayInfo Info;
				Schema->GetGraphDisplayInformation(*Graph, /*out*/ Info);
				return Info.DisplayName;
			}
			else
			{
				// if we don't have a schema, we're dealing with a malformed (or incomplete graph)...
				// possibly in the midst of some transaction - here we return the object's outer path 
				// so we can at least get some context as to which graph we're referring
				return FText::FromString(Graph->GetPathName());
			}
		}
		return LOCTEXT("UnknownGraphName", "Unknown");
	});
	Args.OnGetDocumentState = FOnGetDocumentState::CreateLambda([](const FWorkspaceEditorContext& InContext, TSharedRef<SWidget> InWidget)
	{
		FVector2f ViewLocation = FVector2f::ZeroVector;
		float ZoomAmount = 0.f;
		
		if(const TSharedPtr<SGraphDocument> GraphDocument = StaticCastSharedRef<SGraphDocument>(InWidget))
		{
			GraphDocument->GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);
		}
	
		return TInstancedStruct<FGraphDocumentState>::Make(InContext.Document.GetObject(), InContext.Document.Export, FDeprecateSlateVector2D(ViewLocation), ZoomAmount);
	});
	Args.OnSetDocumentState = FOnSetDocumentState::CreateLambda([](const FWorkspaceEditorContext& InContext, TSharedRef<SWidget> InWidget, const TInstancedStruct<FWorkspaceDocumentState>& InDocumentState)
	{
		if(const FGraphDocumentState* GraphDocumentState = InDocumentState.GetPtr<FGraphDocumentState>())
		{
			if (const TSharedPtr<SGraphDocument> GraphDocument = StaticCastSharedRef<SGraphDocument>(InWidget))
			{
				GraphDocument->GraphEditor->SetViewLocation(GraphDocumentState->ViewLocation, GraphDocumentState->ZoomAmount);
			}
		}
	});

	return Args;
}

bool FWorkspaceEditorModule::GetExportedAssetsForWorkspace(const FAssetData& InWorkspaceAsset, FWorkspaceAssetRegistryExports& OutExports)
{
	constexpr bool bLoadAsset = false;
	if(UWorkspace* Workspace = Cast<UWorkspace>(InWorkspaceAsset.FastGetAsset(bLoadAsset)))
	{
		for(UWorkspaceAssetEntry* WorkspaceAssetEntry : Workspace->AssetEntries)
		{
			OutExports.Assets.Add({WorkspaceAssetEntry->Asset.ToSoftObjectPath()});
		}
	}
	else
	{
		const FString ExternalObjectsPath = FExternalPackageHelper::GetExternalObjectsPath(InWorkspaceAsset.PackageName.ToString());
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths.Add(*ExternalObjectsPath);

		TArray<FAssetData> AssetDataEntries;
		IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
		AssetRegistry.GetAssets(Filter, AssetDataEntries);

		for (const FAssetData& AssetDataEntry : AssetDataEntries)
		{
			FSoftObjectPath SoftObjectPath(AssetDataEntry.GetTagValueRef<FString>(UWorkspaceAssetEntry::ExportsAssetRegistryTag));
			if(SoftObjectPath.IsValid())
			{
				OutExports.Assets.Add({SoftObjectPath});
			}
		}
	}

	return OutExports.Assets.Num() > 0;
}

 IWorkspaceEditor* FWorkspaceEditorModule::OpenWorkspaceForObject(UObject* InObject, EOpenWorkspaceMethod InOpenMethod, const TSubclassOf<UWorkspaceFactory> WorkSpaceFactoryClass/*= UWorkspaceFactory::StaticClass()*/)
{
	UWorkspaceFactory* DefaultWorkspaceFactory = WorkSpaceFactoryClass->GetDefaultObject<UWorkspaceFactory>();
	UWorkspaceSchema* DefaultWorkspaceSchema = DefaultWorkspaceFactory->SchemaClass ? DefaultWorkspaceFactory->SchemaClass->GetDefaultObject<UWorkspaceSchema>() : nullptr;

	if (DefaultWorkspaceSchema && DefaultWorkspaceSchema->CanSaveWorkspace())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	
		TArray<FAssetData> RelevantWorkspaceAssets;
	
		if(InOpenMethod != EOpenWorkspaceMethod::AlwaysOpenNewWorkspace)
		{
			// Look for existing workspaces that export this asset
			FARFilter ARFilter;
			ARFilter.ClassPaths.Add(UWorkspace::StaticClass()->GetClassPathName());
			ARFilter.bRecursiveClasses = true;
	
			TArray<FAssetData> AllWorkspaceAssets;
			AssetRegistryModule.Get().GetAssets(ARFilter, AllWorkspaceAssets);
	
			for(const FAssetData& WorkspaceAsset : AllWorkspaceAssets)
			{
				FWorkspaceAssetRegistryExports Exports;
				GetExportedAssetsForWorkspace(WorkspaceAsset, Exports);
	
				FSoftObjectPath ObjectPath(InObject);
				for(const FWorkspaceAssetRegistryExportEntry& ExportEntry : Exports.Assets)
				{
					if(ExportEntry.Asset == ObjectPath)
					{
						RelevantWorkspaceAssets.Add(WorkspaceAsset);
						break;
					}
				}
			}
		}
	
		IWorkspaceEditor* WorkspaceEditor = nullptr;
	
		auto HandleNewWorkspace = [InObject, &WorkspaceEditor](TObjectPtr<UWorkspace> NewWorkspace)
		{
			NewWorkspace->AddAsset(InObject, false);
			NewWorkspace->MarkPackageDirty();
	
			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			UWorkspaceAssetEditor* AssetEditor = NewObject<UWorkspaceAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
			AssetEditor->SetObjectToEdit(NewWorkspace);
			AssetEditor->Initialize();
			
			WorkspaceEditor = static_cast<IWorkspaceEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(NewWorkspace, true));
		};
	
		auto HandleExistingWorkspace = [InObject, &WorkspaceEditor](TObjectPtr<UWorkspace> ExistingWorkspace)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ExistingWorkspace);
	
			WorkspaceEditor = static_cast<IWorkspaceEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ExistingWorkspace, true));
		};
	
		if(InOpenMethod == EOpenWorkspaceMethod::AlwaysOpenNewWorkspace || RelevantWorkspaceAssets.Num() == 0)
		{
			// No relevant workspaces, so open a new one and add the asset
			UWorkspaceFactory* Factory = NewObject<UWorkspaceFactory>(GetTransientPackage(), WorkSpaceFactoryClass.Get());
			UPackage* Package = CreatePackage(nullptr);
			FName PackageName = *FPaths::GetBaseFilename(Package->GetName());
			if (Factory->ConfigureProperties())
			{
				TObjectPtr<UWorkspace> SelectedWorkspace = CastChecked<UWorkspace>(Factory->FactoryCreateNew(UWorkspace::StaticClass(), Package, PackageName, RF_Public | RF_Standalone, nullptr, GWarn));
				HandleNewWorkspace(SelectedWorkspace);
			}
		}
		else if(RelevantWorkspaceAssets.Num() == 1)
		{
			// One existing workspace, open it
			HandleExistingWorkspace(CastChecked<UWorkspace>(RelevantWorkspaceAssets[0].GetAsset()));
		}
		else
		{
			// Multiple existing workspaces, present a window to let the user choose one to open with
			TSharedRef<SWorkspacePicker> WorkspacePicker = SNew(SWorkspacePicker)
				.WorkspaceAssets(RelevantWorkspaceAssets)
				.OnExistingWorkspaceSelected_Lambda(HandleExistingWorkspace)
				.OnNewWorkspaceCreated_Lambda(HandleNewWorkspace)
				.WorkspaceFactoryClass(WorkSpaceFactoryClass.Get());
	
			WorkspacePicker->ShowModal();
		}
	
		if(WorkspaceEditor)
		{
			WorkspaceEditor->OpenAssets({InObject});
		}
	
		return WorkspaceEditor;
	}
	else
	{
		UWorkspaceFactory* Factory = NewObject<UWorkspaceFactory>(GetTransientPackage(), WorkSpaceFactoryClass.Get());
		if (!Factory->SchemaClass)
		{
			if (!ensureMsgf(Factory->ConfigureProperties(), TEXT("Can not create a workspace without a valid Schema")))
			{
				return nullptr;
			}
		}

		TObjectPtr<UWorkspace> NewWorkspace = CastChecked<UWorkspace>(Factory->FactoryCreateNew(UWorkspace::StaticClass(), GetTransientPackage(), NAME_None, RF_Public | RF_Standalone, nullptr, GWarn));
		
		NewWorkspace->AddAsset(InObject, false);
		(void)NewWorkspace->MarkPackageDirty();
	
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UWorkspaceAssetEditor* AssetEditor = NewObject<UWorkspaceAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->SetObjectToEdit(NewWorkspace);
		AssetEditor->Initialize();
		
		IWorkspaceEditor* WorkspaceEditor = static_cast<IWorkspaceEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InObject, true));
		if (WorkspaceEditor)
		{
			WorkspaceEditor->OpenAssets({InObject});
		}
		
		return WorkspaceEditor;
	}
}

 TSharedPtr<IWorkspacePicker> FWorkspaceEditorModule::CreateWorkspacePicker(const IWorkspacePicker::FConfig& Config) const
 {
	TSharedRef<SWorkspacePicker> WorkspacePicker = SNew(SWorkspacePicker)
		.HintText(Config.HintText)
		.WorkspaceFactoryClass(Config.WorkspaceFactoryClass);

	return WorkspacePicker;
 }

const FObjectDocumentArgs* FWorkspaceEditorModule::FindObjectDocumentType(const FWorkspaceDocument& Document) const
{
	const FObjectDocumentArgs* FoundArgs = ObjectDocumentArgs.Find(Document.GetObject()->GetClass()->GetClassPathName());
	while(FoundArgs && FoundArgs->OnRedirectWorkspaceContext.IsBound())
	{
		UObject* Object = FoundArgs->OnRedirectWorkspaceContext.Execute(Document);
		FoundArgs = Object ? ObjectDocumentArgs.Find(Object->GetClass()->GetClassPathName()) : nullptr;
	}

	return FoundArgs;
}

const FDocumentSubObjectArgs* FWorkspaceEditorModule::FindDocumentSubObjectType(const UObject* InObject) const
{
	return DocumentSubObjectArgs.Find(InObject->GetClass()->GetClassPathName());
}

TArray<FTopLevelAssetPath> FWorkspaceEditorModule::GetAllowedObjectTypesForArea(FName InSpawnLocation) const
{
	return DocumentAreaMap.FindRef(InSpawnLocation).Array();
}

IWorkspaceEditorModule::FOnRegisterDetailCustomizations& FWorkspaceEditorModule::OnRegisterWorkspaceDetailsCustomization()
{
	return OnRegisterDetailCustomizations;
}

void FWorkspaceEditorModule::ApplyWorkspaceDetailsCustomization(const TWeakPtr<IWorkspaceEditor>& InWorkspaceEditor, TSharedPtr<IDetailsView>& DetailsView) const
{
	if (OnRegisterDetailCustomizations.IsBound())
	{
		OnRegisterDetailCustomizations.Broadcast(InWorkspaceEditor, DetailsView);
	}
}

void FWorkspaceEditorModule::RegisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InItemDetailsId, TSharedPtr<IWorkspaceOutlinerItemDetails> InItemDetails)
{
	if (!OutlinerItemDetails.Contains(InItemDetailsId))
	{
		OutlinerItemDetails.Add(InItemDetailsId, InItemDetails);
	}
}

void FWorkspaceEditorModule::UnregisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InItemDetailsId)
{
	OutlinerItemDetails.Remove(InItemDetailsId);
}

TSharedPtr<IWorkspaceOutlinerItemDetails> FWorkspaceEditorModule::GetOutlinerItemDetails(const FOutlinerItemDetailsId& InItemDetailsId)
{
	TSharedPtr<IWorkspaceOutlinerItemDetails>* FoundDetails = OutlinerItemDetails.Find(InItemDetailsId);
	return FoundDetails ? *FoundDetails : nullptr;
}

void FWorkspaceEditorModule::RegisterViewportControllerFactory(const UClass* InClassPath, const FWorkspaceViewportControllerFactory InControllerFactory)
{
	ViewportControllerFactories.Add(InClassPath->GetFName(), InControllerFactory);	
}
	
void FWorkspaceEditorModule::UnregisterViewportControllerFactory(const UClass* InClassPath)
{
	ViewportControllerFactories.Remove(InClassPath->GetFName());
}
	
TUniquePtr<IWorkspaceViewportController> FWorkspaceEditorModule::CreateViewportController(const UClass* InClassPath) const
{
	if (const FWorkspaceViewportControllerFactory* ControllerFactory = ViewportControllerFactories.Find(InClassPath->GetFName()))
	{
		return (*ControllerFactory)();
	}
	return nullptr;
}

}

IMPLEMENT_MODULE(UE::Workspace::FWorkspaceEditorModule, WorkspaceEditor);

#undef LOCTEXT_NAMESPACE