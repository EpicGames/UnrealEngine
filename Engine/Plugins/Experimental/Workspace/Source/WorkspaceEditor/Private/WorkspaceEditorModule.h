// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWorkspaceEditorModule.h"
#include "UObject/TopLevelAssetPath.h"
#include "IWorkspaceOutlinerItemDetails.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

struct FWorkspaceAssetRegistryExports;
class SWorkspaceTabWrapper;

namespace UE::Workspace
{
	struct FAssetDocumentSummoner;
	class FWorkspaceEditor;
	class FWorkspaceOutlinerMode;
}

namespace UE::Workspace
{

class FWorkspaceEditorModule : public IWorkspaceEditorModule
{
private:
	// IModuleInterface interface
	virtual void StartupModule() override;
	
	// IWorkspaceModule interface
	virtual void RegisterObjectDocumentType(const FTopLevelAssetPath& InClassPath, const FObjectDocumentArgs& InParams) override;
	virtual void UnregisterObjectDocumentType(const FTopLevelAssetPath& InClassPath) override;
	virtual void RegisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath, const FDocumentSubObjectArgs& InParams) override;
	virtual void UnregisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath) override;
	virtual FObjectDocumentArgs CreateGraphDocumentArgs(const FGraphDocumentWidgetArgs& InArgs) override;
	virtual IWorkspaceEditor* OpenWorkspaceForObject(UObject* InObject, EOpenWorkspaceMethod InOpenMethod, const TSubclassOf<UWorkspaceFactory> WorkSpaceFactoryClass) override;	
	virtual FOnRegisterDetailCustomizations& OnRegisterWorkspaceDetailsCustomization() override;
	virtual void RegisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InItemDetailsId, TSharedPtr<IWorkspaceOutlinerItemDetails> InItemDetails) override;	
	virtual void UnregisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InItemDetails) override;
	virtual FOnRegisterTabs& OnRegisterTabsForEditor() override { return RegisterTabsForEditor; }
	virtual FOnExtendTabs& OnExtendTabs() override { return ExtendTabsForEditor; }
	virtual FOnExtendToolMenuContext& OnExtendToolMenuContext() override { return ExtendToolMenuContext; }
	virtual TSharedPtr<IWorkspacePicker> CreateWorkspacePicker(const IWorkspacePicker::FConfig& Config) const override;
	virtual void RegisterViewportControllerFactory(const UClass* InClassPath, const FWorkspaceViewportControllerFactory InControllerFactory) override;
	virtual void UnregisterViewportControllerFactory(const UClass* InClassPath) override;

	// Create a new viewport controller for a given asset class if a controller has been registered
	// * Can return nullptr
	TUniquePtr<IWorkspaceViewportController> CreateViewportController(const UClass* InClassPath) const;

	// Find an existing registered object document type. Note this redirects based on FObjectDocumentArgs::OnRedirectWorkspaceContext
	const FObjectDocumentArgs* FindObjectDocumentType(const FWorkspaceDocument& Document) const;

	// Find an existing registered document sub-object type.
	const FDocumentSubObjectArgs* FindDocumentSubObjectType(const UObject* InObject) const;

	// Find the set of allowed object types for the specified spawn location
	TArray<FTopLevelAssetPath> GetAllowedObjectTypesForArea(FName InSpawnLocation) const;

	// Get the exported set of assets in asset registry tags for a workspace's FAssetData
	static bool GetExportedAssetsForWorkspace(const FAssetData& InWorkspaceAsset, FWorkspaceAssetRegistryExports& OutExports);

	// Applies any previously registered details-view customizations
	void ApplyWorkspaceDetailsCustomization(const TWeakPtr<IWorkspaceEditor>& Editor, TSharedPtr<IDetailsView>& DetailsView) const;

	TMap<FTopLevelAssetPath, FObjectDocumentArgs> ObjectDocumentArgs;

	TMap<FName, FWorkspaceViewportControllerFactory> ViewportControllerFactories;
	
	TMap<FName, TSet<FTopLevelAssetPath>> DocumentAreaMap;

	TMap<FTopLevelAssetPath, FDocumentSubObjectArgs> DocumentSubObjectArgs; 

	static TMap<FOutlinerItemDetailsId, TSharedPtr<IWorkspaceOutlinerItemDetails>> OutlinerItemDetails;
	static TSharedPtr<IWorkspaceOutlinerItemDetails> GetOutlinerItemDetails(const FOutlinerItemDetailsId& DetailId);

	/** Event called to allow external clients to register details customizations */
	FOnRegisterDetailCustomizations OnRegisterDetailCustomizations;

	/** Event called to allow external clients to register additional tabs for the specified editor */
	FOnRegisterTabs RegisterTabsForEditor;
	
	/** Event called to allow external clients to extend the tab layout for the specified editor */
	FOnExtendTabs ExtendTabsForEditor;

	/** Event called to allow external clients to add details to the ToolMenuContext used for the toolbar and context menus */
	FOnExtendToolMenuContext ExtendToolMenuContext;

	friend struct FAssetDocumentSummoner;
	friend struct FWorkspaceOutlinerTreeItem;
	friend class SWorkspaceOutlinerTreeLabel;
	friend class FWorkspaceEditor;
	friend class FWorkspaceOutlinerMode;
	friend class FWorkspaceOutlinerSourceControlColumn;
	friend class FWorkspaceOutlinerFileStateColumn;
	friend class FWorkspaceAssetReferenceOutlinerItemDetails;
	friend class FWorkspaceOutlinerHierarchy;
	friend struct FWorkspaceDocument;
	friend SWorkspaceTabWrapper;
	friend FWorkspaceEditorContext;
};
	
}
