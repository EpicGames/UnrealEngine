// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceFactory.h"

#include "ClassViewerFilter.h"
#include "DefaultWorkspaceSchema.h"
#include "Workspace.h"
#include "WorkspaceSchema.h"
#include "Kismet2/SClassPickerDialog.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceFactory)

#define LOCTEXT_NAMESPACE "WorkspaceFactory"

UWorkspaceFactory::UWorkspaceFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UWorkspace::StaticClass();
}

bool UWorkspaceFactory::ConfigureProperties()
{
	if(SchemaClass == nullptr)
	{
		class FSchemaFilter : public IClassViewerFilter
		{
		public:
			virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
			{
				return !InClass->HasAnyClassFlags(CLASS_Abstract) && InClass->IsChildOf(UWorkspaceSchema::StaticClass());
			}

			virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
			{
				return false;
			}
		};

		FClassViewerInitializationOptions Options;
		Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
		Options.bEditorClassesOnly = true;
		Options.bExpandAllNodes = true;
		Options.InitiallySelectedClass = UDefaultWorkspaceSchema::StaticClass();
		Options.ClassFilters.Add(MakeShared<FSchemaFilter>());

		UClass* ChosenSchema = nullptr;
		if(SClassPickerDialog::PickClass(LOCTEXT("ChooseSchema", "Choose a Workspace Schema"), Options, ChosenSchema, UWorkspaceSchema::StaticClass()))
		{
			SchemaClass = ChosenSchema;
		}
	}

	return SchemaClass != nullptr;
}

UObject* UWorkspaceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	UWorkspace* NewWorkspace = NewObject<UWorkspace>(InParent, Class, Name, Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted);
	ensureMsgf(SchemaClass != nullptr, TEXT("UWorkspaceFactory requires a valid Schema Class"));
	NewWorkspace->SchemaClass = SchemaClass;
	NewWorkspace->Guid = FGuid::NewGuid();

	// make sure the package is never cooked.
	UPackage* Package = NewWorkspace->GetOutermost();
	Package->SetPackageFlags(Package->GetPackageFlags() | PKG_EditorOnly);

	return NewWorkspace;
}

#undef LOCTEXT_NAMESPACE
