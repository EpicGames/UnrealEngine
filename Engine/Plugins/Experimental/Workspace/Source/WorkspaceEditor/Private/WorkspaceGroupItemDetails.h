// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IWorkspaceOutlinerItemDetails.h"
#include "WorkspaceEditorModule.h"
#include "WorkspaceItemMenuContext.h"

class UUAFRigVMAssetEditorData;
enum class EAnimNextEditorDataNotifType : uint8;

namespace UE::Workspace
{
class FWorkspaceGroupOutlinerItemDetails : public IWorkspaceOutlinerItemDetails
{
public:
	virtual ~FWorkspaceGroupOutlinerItemDetails() override = default;
	
	virtual FString GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const override
	{
		const FWorkspaceOutlinerGroupItemData& Data = Export.GetData().Get<FWorkspaceOutlinerGroupItemData>();		
		return Data.GroupName;
	}

	virtual bool HandleSelected(const FToolMenuContext& ToolMenuContext) const override { return true; }
	
	virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override
	{
		const FWorkspaceOutlinerGroupItemData& Data = Export.GetData().Get<FWorkspaceOutlinerGroupItemData>();
		return &Data.GroupIcon;
	}

	virtual bool IsExpandedByDefault() const override
	{
		return false;
	}	
};

}
