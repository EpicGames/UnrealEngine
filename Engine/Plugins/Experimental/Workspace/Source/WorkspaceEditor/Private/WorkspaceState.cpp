// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceState.h"

#include "JsonObjectConverter.h"
#include "Workspace.h"
#include "WorkspaceDocumentState.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorkspaceState)

UWorkspaceState::UWorkspaceState()
{
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetRenamed().AddUObject(this, &UWorkspaceState::HandleAssetRenamed);
	}
}

FString UWorkspaceState::GetWorkspacePath(const UWorkspace* InWorkspace)
{
	// Save the file to a specific directory, keyed by GUID
	TStringBuilder<512> PathBuilder;
	PathBuilder.Append(FPaths::ProjectSavedDir());
	PathBuilder.Append(TEXT("Config/Workspace/"));
	InWorkspace->Guid.AppendString(PathBuilder);
	PathBuilder.Append(TEXT(".json"));
	return PathBuilder.ToString();
}

void UWorkspaceState::SaveToJson(const UWorkspace* InWorkspace)
{
	// Record the path we saved this with
	WorkspacePath = FSoftObjectPath(InWorkspace);

	FString String;
	const FString FileName = GetWorkspacePath(InWorkspace);
	if(FJsonObjectConverter::UStructToJsonObjectString(StaticClass(), this, String))
	{
		FFileHelper::SaveStringToFile(String, *FileName);
	}
}

void UWorkspaceState::LoadFromJson(const UWorkspace* InWorkspace)
{
	FString String;
	const FString FileName = GetWorkspacePath(InWorkspace);
	if(FFileHelper::LoadFileToString(String, *FileName))
	{
		FJsonObjectConverter::JsonObjectStringToUStruct<UWorkspaceState>(String, this);
	}

	// Record the path we loaded this with
	WorkspacePath = FSoftObjectPath(InWorkspace);
}

void UWorkspaceState::HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldName)
{
	const FSoftObjectPath OldPath(*InOldName);

	// See if any of our assets were renamed and update accordingly
	for(TInstancedStruct<FWorkspaceDocumentState>& DocumentState : DocumentStates)
	{
		FSoftObjectPath& DocumentPath = DocumentState.GetMutable().Object;
		if(DocumentPath == OldPath)
		{
			DocumentPath = InAssetData.GetSoftObjectPath();
		}
	}
}
