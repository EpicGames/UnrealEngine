// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "WorkspaceState.generated.h"

class UWorkspace;
struct FWorkspaceDocumentState;

namespace UE::Workspace
{
	class FWorkspaceEditor;
	class FWorkspaceEditorModule;
}

UCLASS()
class UWorkspaceState : public UObject
{
	GENERATED_BODY()

	UWorkspaceState();

	// Generate a file path used to persist workspace state
	static FString GetWorkspacePath(const UWorkspace* InWorkspace);

	// Save this workspace state to JSON
	void SaveToJson(const UWorkspace* InWorkspace);

	// Load this workspace state from JSON
	void LoadFromJson(const UWorkspace* InWorkspace);

	// Update soft object paths to any assets we reference
	void HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldName);

	// Accessors for user state
	void SetUserState(FInstancedStruct&& InUserState) { UserState = MoveTemp(InUserState); }
	const FInstancedStruct& GetUserState() const { return UserState; }

	// Path to the workspace we are persisting, to allow files to be more easily parsed out
	UPROPERTY()
	FSoftObjectPath WorkspacePath;

	// User workspace state
	UPROPERTY()
	FInstancedStruct UserState;

	// All the workspace documents we are persisting
	UPROPERTY()
	TArray<TInstancedStruct<FWorkspaceDocumentState>> DocumentStates;

	friend class UE::Workspace::FWorkspaceEditor;
	friend class UE::Workspace::FWorkspaceEditorModule;
	friend class UWorkspace;
};