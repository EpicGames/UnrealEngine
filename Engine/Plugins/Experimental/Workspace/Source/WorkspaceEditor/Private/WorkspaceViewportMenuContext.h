// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"

#include "WorkspaceViewportMenuContext.generated.h"

class UWorkspaceViewportSceneDescription;

UCLASS()
class UWorkspaceViewportMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TAttribute<bool> bIsPinned;
	TAttribute<FSoftObjectPath> PreviewAssetPath;
	FSimpleDelegate OnPinnedClicked;

	UPROPERTY()
	TObjectPtr<UWorkspaceViewportSceneDescription> SceneDescription;
};