// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkspaceViewportSceneDescription.h"

FSimpleMulticastDelegate& UWorkspaceViewportSceneDescription::GetOnConfigChanged()
{
	return OnConfigChanged;
}

void UWorkspaceViewportSceneDescription::BroadcastOnConfigChanged()
{
	OnConfigChanged.Broadcast();
}