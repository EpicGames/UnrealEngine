// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/BaseAssetToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API WORKSPACEEDITOR_API

class UWorkspaceViewportSceneDescription;
class UWorkspaceAssetEditor;
class UWorkspaceSchema;
class UWorkspace;
struct FWorkspaceOutlinerItemExport;

namespace UE::Workspace
{

typedef TWeakPtr<SWidget> FGlobalSelectionId;
using FOnClearGlobalSelection = FSimpleDelegate;

struct FWorkspaceDocument;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnFocusedDocumentChanged, const FWorkspaceDocument&);

// RAII helper allowing for a multi-widget selection scope within a WorkspaceEditor instance
struct FWorkspaceEditorSelectionScope
{
	UE_API FWorkspaceEditorSelectionScope(const TSharedPtr<class IWorkspaceEditor>& InWorkspaceEditor);	
	UE_API ~FWorkspaceEditorSelectionScope();

	TWeakPtr<class IWorkspaceEditor> WeakWorkspaceEditor; 
};

class IWorkspaceEditor : public FBaseAssetToolkit
{
public:
	IWorkspaceEditor(UAssetEditor* InOwningAssetEditor) : FBaseAssetToolkit(InOwningAssetEditor) {}

	// Open the supplied assets for editing within the workspace editor
	virtual void OpenAssets(TConstArrayView<FAssetData> InAssets) = 0;

	// Open the supplied exports for editing within the workspace editor
	virtual void OpenExports(TConstArrayView<FWorkspaceOutlinerItemExport> InExports, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride = TOptional<FDocumentTracker::EOpenDocumentCause>()) = 0;

	// Open the supplied objects for editing within the workspace editor
	virtual void OpenObjects(TConstArrayView<UObject*> InObjects, TOptional<FDocumentTracker::EOpenDocumentCause> OpenCauseOverride = TOptional<FDocumentTracker::EOpenDocumentCause>()) = 0;

	// Get the current set of opened (loaded) assets of the specified class
	virtual void GetOpenedAssetsOfClass(TSubclassOf<UObject> InClass, TArray<UObject*>& OutObjects) const = 0;

	// Get the current set of opened (loaded) assets 
	void GetOpenedAssets(TArray<UObject*>& OutObjects) const { GetOpenedAssetsOfClass(UObject::StaticClass(), OutObjects); }

	// Get the current set of opened (loaded) assets
	template<typename AssetClass>
	void GetOpenedAssets(TArray<UObject*>& OutObjects) const { GetOpenedAssetsOfClass(AssetClass::StaticClass(), OutObjects); }

	// Get the current set of assets in this workspace editor 
	virtual void GetAssets(TArray<FAssetData>& OutAssets, bool bIncludeAssetReferences = false) const = 0;

	// Close the supplied objects if they are open for editing within the workspace editor
	virtual void CloseObjects(TConstArrayView<UObject*> InObjects) = 0;

	// Show the supplied objects in the workspace editor details panel
	virtual void SetDetailsObjects(const TArray<UObject*>& InObjects) = 0;

	// Refresh the workspace editor details panel
	virtual void RefreshDetails() = 0;

	// Exposes the editor WorkspaceSchema
	virtual UWorkspaceSchema* GetSchema() const = 0;

	// Set the _current_ global selection (last SWidget with selection set) with delegate to clear it selection on next SetGlobalSelection()
	virtual void SetGlobalSelection(FGlobalSelectionId SelectionId, FOnClearGlobalSelection OnClearSelectionDelegate) = 0;

	// Get the currently focused document. @return unset if the class does not match or no document is focused
	virtual const FWorkspaceDocument& GetFocusedDocumentOfClass(const TObjectPtr<UClass> InClass) const = 0;

	// Returns the currently focused workspace document (last user activated tab), unset in case of none is focused
	virtual const FWorkspaceDocument& GetFocusedWorkspaceDocument() const = 0;

	// Multicast delegate broadcast whenever the document focus of the WorkspaceEditor changes
	virtual FOnFocusedDocumentChanged& OnFocusedDocumentChanged() = 0;

	// Get the current single selection of the outliner.
	// @return true if a single selection is active
	virtual bool GetOutlinerSelection(TArray<FWorkspaceOutlinerItemExport>& OutExports) const = 0;

	// Delegate fired when selection changes in the workspace outliner
	using FOnOutlinerSelectionChanged = TMulticastDelegate<void(TConstArrayView<FWorkspaceOutlinerItemExport> InExports)>;
	virtual FOnOutlinerSelectionChanged& OnOutlinerSelectionChanged() = 0;

	// Retrieves the common DetailsView widget
	virtual TSharedPtr<IDetailsView> GetDetailsView() = 0;

	// Returns the workspace asset
	virtual UObject* GetWorkspaceAsset() const = 0;

	// Returns the name of the package where the workspace is located
	virtual FString GetPackageName() const = 0;

	// Name of EditorToolkit, used for testing against FAssetEditorToolkit::GetToolkitFName 
	static FName GetWorkspaceEditorToolkitName() { return FName("WorkspaceEditor"); }

	// Set a new scene description config
	virtual void SetSceneDescription(TObjectPtr<UWorkspaceViewportSceneDescription> InSceneDescription) = 0;

	// Get the current scene description config
	virtual UWorkspaceViewportSceneDescription* GetSceneDescription() const = 0;

	// Gets the currently focused asset's UObject
	virtual UObject* GetFocusedAsset() const = 0;
};

}

#undef UE_API
