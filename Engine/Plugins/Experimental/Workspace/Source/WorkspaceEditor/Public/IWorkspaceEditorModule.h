// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorModes.h"
#include "GraphEditor.h"
#include "StructUtils/InstancedStruct.h"
#include "WorkspaceFactory.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleInterface.h"
#include "IWorkspaceOutlinerItemDetails.h"
#include "IWorkspacePicker.h"
#include "WorkspaceViewportController.h"

#define UE_API WORKSPACEEDITOR_API

class SWidget;
class UEdGraph;
class IDetailsView;
class FTabManager;
class FWorkflowAllowedTabSet;
class IDetailCustomization;
class FLayoutExtender;
class FAdvancedPreviewScene;

struct FSlateBrush;
struct FTopLevelAssetPath;
struct FWorkspaceDocumentState;
struct FRegisterCustomClassLayoutParams;


namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::Workspace
{

namespace WorkspaceTabs
{
	WORKSPACEEDITOR_API extern const FName TopLeftDocumentArea;
	WORKSPACEEDITOR_API extern const FName BottomLeftDocumentArea;
	WORKSPACEEDITOR_API extern const FName TopMiddleDocumentArea;
	WORKSPACEEDITOR_API extern const FName BottomMiddleDocumentArea;
	WORKSPACEEDITOR_API extern const FName TopRightDocumentArea;
	WORKSPACEEDITOR_API extern const FName BottomRightDocumentArea;
	WORKSPACEEDITOR_API extern const FName RightSidebarDocumentArea;
}

struct FWorkspaceDocument
{
	FWorkspaceOutlinerItemExport Export;
	UObject* Object = nullptr;
	
	FWorkspaceDocument() = default;
	UE_API FWorkspaceDocument(const FWorkspaceOutlinerItemExport& InExport, UObject* InObject);

	bool operator==(const FWorkspaceDocument& InOther) const
	{
		// Comparing object and export path (disregarding export data struct)
		return Object == InOther.Object && GetTypeHash(Export) == GetTypeHash(InOther.Export);
	}

	UObject* GetObject() const { return Object; }

	template<class ObjectClass>
	ObjectClass* GetTypedObject() const { return Cast<ObjectClass>(Object); }

	template<class ObjectClass>
	ObjectClass* GetTypedObjectChecked() const { return CastChecked<ObjectClass>(Object); }
};

// Context passed to workspace editor delegates
struct FWorkspaceEditorContext
{
	UE_API FWorkspaceEditorContext(const TSharedRef<IWorkspaceEditor>& InWorkspaceEditor, const FWorkspaceDocument& InDocument);
	UE_API FWorkspaceEditorContext(const TSharedRef<IWorkspaceEditor>& InWorkspaceEditor, UObject* InObject, const FWorkspaceOutlinerItemExport& InExport);
	FWorkspaceEditorContext() = delete;

	// The current workspace editor
	TSharedRef<IWorkspaceEditor> WorkspaceEditor;

	// The document (object + export) being edited
	FWorkspaceDocument Document;
};

struct FWorkspaceBreadcrumb : TSharedFromThis<FWorkspaceBreadcrumb>
{
	using FOnGetBreadcrumbLabel = TDelegate<TAttribute<FText>()>;
	using FOnBreadcrumbClicked = TDelegate<void()>;
	using FCanSaveBreadcrumb = TDelegate<bool()>;
	using FOnSaveBreadcrumb = TDelegate<void()>;
	
	FOnGetBreadcrumbLabel OnGetLabel;
	FOnBreadcrumbClicked OnClicked;
	FCanSaveBreadcrumb CanSave;
	FOnSaveBreadcrumb OnSave;
};

using FOnRedirectWorkspaceContext = TDelegate<UObject*(const FWorkspaceDocument&)>;

using FOnMakeDocumentWidget = TDelegate<TSharedRef<SWidget>(const FWorkspaceEditorContext&)>;

using FOnGetTabIcon = TDelegate<const FSlateBrush*(const FWorkspaceEditorContext&)>;

using FOnGetTabName = TDelegate<TAttribute<FText>(const FWorkspaceEditorContext&)>;

using FOnGetDocumentState = TDelegate<TInstancedStruct<FWorkspaceDocumentState>(const FWorkspaceEditorContext&, TSharedRef<SWidget>)>;

using FOnSetDocumentState = TDelegate<void(const FWorkspaceEditorContext&, TSharedRef<SWidget>, const TInstancedStruct<FWorkspaceDocumentState>&)>;

using FOnGetDocumentBreadcrumbTrail = TDelegate<void(const FWorkspaceEditorContext&, TArray<TSharedPtr<FWorkspaceBreadcrumb>>&)>;

using FOnGetDocumentForSubObject = TDelegate<UObject*(const UObject*)>;

using FOnPostDocumentOpenedForSubObject = TDelegate<void(const FWorkspaceEditorContext&, TSharedRef<SWidget>, UObject*)>;

// Arguments used to make document widgets for objects
struct FObjectDocumentArgs
{
	FObjectDocumentArgs() = default;

	FObjectDocumentArgs(const FOnMakeDocumentWidget& InOnMakeDocumentWidget, FName InSpawnLocation = WorkspaceTabs::TopMiddleDocumentArea)
		: OnMakeDocumentWidget(InOnMakeDocumentWidget)
		, SpawnLocation(InSpawnLocation)
	{}

	FObjectDocumentArgs(const FOnRedirectWorkspaceContext& InOnRedirectWorkspaceContext)
		: OnRedirectWorkspaceContext(InOnRedirectWorkspaceContext)
	{}
	
	// Delegate called to redirect the context to another document object (e.g. a subobject)
	FOnRedirectWorkspaceContext OnRedirectWorkspaceContext;

	// Delegate called to generate a widget for the supplied object
	FOnMakeDocumentWidget OnMakeDocumentWidget;

	// Delegate called to build a struct used to store the document's state
	FOnGetDocumentState OnGetDocumentState;

	// Delegate called to use a struct to restore the document's state
	FOnSetDocumentState OnSetDocumentState;

	// Delegate called to get the tab icon to display. If this is unset, the icon will default to the asset icon for the class
	FOnGetTabIcon OnGetTabIcon;

	// Delegate called to get the tab name to display. If this is unset, the object's name will be used
	FOnGetTabName OnGetTabName;

	// Where to spawn the widget in the workspace layout - e.g. one of WorkspaceTabs
	FName SpawnLocation = WorkspaceTabs::TopMiddleDocumentArea;

	// Delegate called to get the bread crumb trail for this document tab
	FOnGetDocumentBreadcrumbTrail OnGetDocumentBreadcrumbTrail;

	// EditorMode ID to be associated with this asset, will be used to try and activate matching editor mode when this asset is focused
	FEditorModeID DocumentEditorMode = NAME_None;
};

// Arguments used to open documents for specific subobject types
struct FDocumentSubObjectArgs
{
	// Delegate called to get a document to open for a document's subobject (e.g. a UEdGraphNode could return its containing UEdGraph)
	FOnGetDocumentForSubObject OnGetDocumentForSubObject;

	// Delegate called after a document is opened to process the supplied subobject (e.g. focus it).
	// The widget supplied is the document widget provided via OnMakeDocumentWidget (or a SGraphEditor for FGraphDocumentWidgetArgs)
	FOnPostDocumentOpenedForSubObject OnPostDocumentOpenedForSubObject;
};

using FOnGraphSelectionChanged = TDelegate<void(const FWorkspaceEditorContext&, const FGraphPanelSelectionSet&)>;

using FOnCreateActionMenu = TDelegate<FActionMenuContent(const FWorkspaceEditorContext&, UEdGraph*, const FVector2D&, const TArray<UEdGraphPin*>&, bool, SGraphEditor::FActionMenuClosed)>;

using FOnNodeTextCommitted = TDelegate<void(const FWorkspaceEditorContext&, const FText&, ETextCommit::Type, UEdGraphNode*)>;

using FOnCanPerformActionOnSelectedNodes = TDelegate<bool(const FWorkspaceEditorContext&, const FGraphPanelSelectionSet&)>;
using FOnPerformActionOnSelectedNodes = TDelegate<void(const FWorkspaceEditorContext&, const FGraphPanelSelectionSet&)>;

using FOnCanPasteNodes = TDelegate<bool(const FWorkspaceEditorContext&, const FString&)>;
using FOnPasteNodes = TDelegate<void(const FWorkspaceEditorContext&, const FVector2D&, const FString&)>;

using FOnDuplicateSelectedNodes = TDelegate<void(const FWorkspaceEditorContext&, const FVector2D&, const FGraphPanelSelectionSet&)>;

using FOnGetWorkspaceDetailCustomizationInstance = TDelegate<TSharedRef<IDetailCustomization>(TWeakPtr<IWorkspaceEditor>)>;

using FOnNodeDoubleClicked = TDelegate<void(const FWorkspaceEditorContext&, const UEdGraphNode* InNode)>;

using FOnGraphDocumentCreated = TDelegate<void(const FWorkspaceEditorContext&, TSharedPtr<SWidget> InGraphWidget)>;

using FOnCreateComment = TDelegate<void(const FWorkspaceEditorContext&, const FVector2f&)>;

// Arguments used to make document widgets for graphs
struct FGraphDocumentWidgetArgs
{
	// Where to spawn the widget in the workspace layout - e.g. one of WorkspaceTabs
	FName SpawnLocation = WorkspaceTabs::TopMiddleDocumentArea;

	FOnCreateActionMenu OnCreateActionMenu;

	FOnNodeTextCommitted OnNodeTextCommitted;

	FOnCanPerformActionOnSelectedNodes OnCanCutSelectedNodes;
	FOnPerformActionOnSelectedNodes OnCutSelectedNodes;

	FOnCanPerformActionOnSelectedNodes OnCanCopySelectedNodes;
	FOnPerformActionOnSelectedNodes OnCopySelectedNodes;

	FOnCanPasteNodes OnCanPasteNodes;
	FOnPasteNodes OnPasteNodes;

	FOnCanPerformActionOnSelectedNodes OnCanDeleteSelectedNodes;
	FOnPerformActionOnSelectedNodes OnDeleteSelectedNodes;

	FOnCanPerformActionOnSelectedNodes OnCanDuplicateSelectedNodes;
	FOnDuplicateSelectedNodes OnDuplicateSelectedNodes;

	FOnCanPerformActionOnSelectedNodes OnCanSelectAllNodes;
	FOnPerformActionOnSelectedNodes OnSelectAllNodes;

	FOnGraphSelectionChanged OnGraphSelectionChanged;

	FOnNodeDoubleClicked OnNodeDoubleClicked;

	FOnGraphDocumentCreated OnGraphDocumentCreated;

	FOnCanPerformActionOnSelectedNodes OnCanOpenInNewTab;
	FOnPerformActionOnSelectedNodes OnOpenInNewTab;

	FOnCreateComment OnCreateComment;
};

// Enum describing how to open a workspace
enum class EOpenWorkspaceMethod : int32
{
	// If the asset is already used in a workspace, open that (if not already opened)
	// If the asset is already used in more than one workspace, let the user choose the workspace to open it in
	// If the asset is not yet in a workspace, create a default workspace, add the asset and open the workspace
	Default,

	// Always open a new workspace asset and add the asset to it
	AlwaysOpenNewWorkspace,
};

class IWorkspaceEditorModule : public IModuleInterface
{
public:
	// Open an object inside a workspace editor.
	virtual IWorkspaceEditor* OpenWorkspaceForObject(UObject* InObject, EOpenWorkspaceMethod InOpenMethod, const TSubclassOf<UWorkspaceFactory> WorkSpaceFactoryClass = UWorkspaceFactory::StaticClass()) = 0;

	// Register a widget factory method to spawn for a particular class
	virtual void RegisterObjectDocumentType(const FTopLevelAssetPath& InClassPath, const FObjectDocumentArgs& InArgs) = 0;

	// Unregister a widget factory method to spawn for a particular class
	virtual void UnregisterObjectDocumentType(const FTopLevelAssetPath& InClassPath) = 0;

	// Register a document subobject - an object that opens in the context of another outer document (e.g. A UEdGraphNode in a UEdGraph)
	virtual void RegisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath, const FDocumentSubObjectArgs& InParams) = 0;

	// Unregister a document subobject - an object that opens in the context of another outer document (e.g. A UEdGraphNode in a UEdGraph)
	virtual void UnregisterDocumentSubObjectType(const FTopLevelAssetPath& InClassPath) = 0;

	// Make the required args for a document widget for a UEdGraph
	virtual FObjectDocumentArgs CreateGraphDocumentArgs(const FGraphDocumentWidgetArgs& InArgs) = 0;

	// Event to allow registering details customizations
	DECLARE_EVENT_TwoParams(IWorkspaceEditorModule, FOnRegisterDetailCustomizations, const TWeakPtr<IWorkspaceEditor>&, TSharedPtr<IDetailsView>&);
	virtual FOnRegisterDetailCustomizations& OnRegisterWorkspaceDetailsCustomization() = 0;

	virtual void RegisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InDetailsId, TSharedPtr<IWorkspaceOutlinerItemDetails> InDetails) = 0;
	virtual void UnregisterWorkspaceItemDetails(const FOutlinerItemDetailsId& InDetailsId) = 0;

	// Event to allow registering tabs to other elements
	DECLARE_EVENT_ThreeParams(IWorkspaceEditorModule, FOnRegisterTabs, FWorkflowAllowedTabSet& TabFactories, const TSharedRef<FTabManager>&, TSharedPtr<IWorkspaceEditor>);
	virtual FOnRegisterTabs& OnRegisterTabsForEditor() = 0;

	// Event to allow extending the layout
	DECLARE_EVENT_TwoParams(IWorkspaceEditorModule, FOnExtendTabs, FLayoutExtender&, TSharedPtr<IWorkspaceEditor>);
	virtual FOnExtendTabs& OnExtendTabs() = 0;

	// Event to allow extending the FToolMenuContext
	DECLARE_EVENT_TwoParams(IWorkspaceEditorModule, FOnExtendToolMenuContext, TSharedPtr<IWorkspaceEditor>, FToolMenuContext&);
	virtual FOnExtendToolMenuContext& OnExtendToolMenuContext() = 0;

	virtual TSharedPtr<IWorkspacePicker> CreateWorkspacePicker(const IWorkspacePicker::FConfig& Config) const = 0;

	// Register a viewport controller factory for a given asset class
	// * Viewport controllers are created per viewport
	virtual void RegisterViewportControllerFactory(const UClass* InClass, const FWorkspaceViewportControllerFactory InControllerFactory) = 0;

	// Unregister the viewport controller factory for a given asset class
	virtual void UnregisterViewportControllerFactory(const UClass* InClass) = 0;
};

}

#undef UE_API
