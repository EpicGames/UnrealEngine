// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceAssetRegistryInfo.h"
#include "Styling/SlateColor.h"

class UPackage;
struct FToolMenuContext;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "IWorkspaceOutlinerItemDetails"

namespace UE::Workspace
{
typedef FName FOutlinerItemDetailsId;
static FOutlinerItemDetailsId MakeOutlinerDetailsId(const FWorkspaceOutlinerItemExport& InExport)
{
    return InExport.HasData() ? InExport.GetData().GetScriptStruct()->GetFName() : NAME_None;
}

class IWorkspaceOutlinerItemDetails : public TSharedFromThis<IWorkspaceOutlinerItemDetails>
{
public:
    virtual ~IWorkspaceOutlinerItemDetails() = default;
	virtual FString GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const
	{			
		return Export.GetIdentifier().ToString();
	}

	virtual FText GetToolTipText(const FWorkspaceOutlinerItemExport& Export) const
	{			
		return FText::Format(LOCTEXT("WorkspaceOutlinerToolTipFormat","Identifier: {0}\nPath: {1}\nType: {2}"), FText::FromString(GetDisplayString(Export)), FText::FromString(Export.GetFirstAssetPath().ToString()), Export.HasData() ? FText::FromString(Export.GetData().GetScriptStruct()->GetName()) : LOCTEXT("NoneTypeLabel", "None"));
	}
	
    virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const { return nullptr; }
	virtual FSlateColor GetItemColor(const FWorkspaceOutlinerItemExport& Export) const { return FSlateColor::UseForeground(); }
    virtual bool HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const { return false; }
	virtual bool CanDelete(const FWorkspaceOutlinerItemExport& Export) const { return false; }
	virtual void Delete(TConstArrayView<FWorkspaceOutlinerItemExport> Exports) const {}
    virtual bool CanRename(const FWorkspaceOutlinerItemExport& Export) const { return false; }
    virtual void Rename(const FWorkspaceOutlinerItemExport& Export, const FText& InName) const {}
    virtual bool ValidateName(const FWorkspaceOutlinerItemExport& Export, const FText& InName, FText& OutErrorMessage) const { return false; }
    virtual UPackage* GetPackage(const FWorkspaceOutlinerItemExport& Export) const
	{
		const FSoftObjectPath AssetPath = Export.GetFirstAssetPath();
		if (AssetPath.IsValid())
		{
			if (UObject* AssetObject = AssetPath.ResolveObject())
			{
				return AssetObject->GetPackage();
			}
		}
		return nullptr;
	}
	virtual bool HandleSelected(const FToolMenuContext& ToolMenuContext) const { return false; }
	virtual bool IsExpandedByDefault() const { return true; }
};

}

#undef LOCTEXT_NAMESPACE // "IWorkspaceOutlinerItemDetails"
