// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkspaceFactory.h"

class UWorkspace;

namespace UE::Workspace
{
	class IWorkspacePicker
	{
	public:
		virtual ~IWorkspacePicker() = default;

		enum class EHintText
		{
			SelectedAssetIsPartOfMultipleWorkspaces,
			CreateOrUseExistingWorkspace
		};
		
		struct FConfig
		{
			EHintText HintText = EHintText::SelectedAssetIsPartOfMultipleWorkspaces;
			TSubclassOf<UWorkspaceFactory> WorkspaceFactoryClass = UWorkspaceFactory::StaticClass();
		};
		
		virtual void ShowModal() = 0;

		virtual TObjectPtr<UObject> GetSelectedWorkspace() const = 0;
	};
}