// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAssetEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

#define UE_API WORKSPACEEDITOR_API

class UWorkspaceViewportSceneDescription;

namespace UE::Workspace
{
	class SWorkspaceViewport : public SAssetEditorViewport, public ICommonEditorViewportToolbarInfoProvider, public FGCObject
	{
	public:
		SLATE_BEGIN_ARGS(SWorkspaceViewport){}
			SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
			SLATE_ARGUMENT(TSharedPtr<FEditorViewportClient>, ViewportClient)
			SLATE_ARGUMENT(UWorkspaceViewportSceneDescription*, SceneDescription)
			SLATE_ATTRIBUTE(FSoftObjectPath, PreviewAssetPath)
			SLATE_ATTRIBUTE(bool, bIsPinned)
			SLATE_EVENT(FSimpleDelegate, OnPinnedClicked)
		SLATE_END_ARGS()
		
		TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit = nullptr;

		UE_API void Construct(const FArguments& InArgs);
		
		// SEditorViewport interface
		UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
		UE_API virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		
		// ICommonEditorViewportToolbarInfoProvider interface
		UE_API virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
		UE_API virtual TSharedPtr<FExtender> GetExtenders() const override;
		UE_API virtual void OnFloatingButtonClicked() override;

		UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
		UE_API virtual FString GetReferencerName() const override
		{
			return TEXT("UE::Workspace::SWorkspaceViewport");
		}

	private:
		TAttribute<FSoftObjectPath> PreviewAssetPath;
		TAttribute<bool> bIsPinned;
		FSimpleDelegate OnPinnedClicked;

		TObjectPtr<UWorkspaceViewportSceneDescription> SceneDescription;
	};
}

#undef UE_API