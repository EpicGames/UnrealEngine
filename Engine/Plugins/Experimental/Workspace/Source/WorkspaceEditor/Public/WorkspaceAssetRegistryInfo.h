// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Styling/SlateBrush.h"
#include "WorkspaceAssetRegistryInfo.generated.h"

USTRUCT()
struct FWorkspaceOutlinerItemData
{
	GENERATED_BODY()

	FWorkspaceOutlinerItemData() = default;
};

USTRUCT()
struct FOutlinerItemPath
{
	GENERATED_BODY()
	
	friend struct FWorkspaceOutlinerItemExport;
protected:
	UPROPERTY()
	TArray<FName> PathSegments;

public:
	static FOutlinerItemPath MakePath(const FSoftObjectPath& InSoftObjectPath)
	{
		FOutlinerItemPath Path;
		Path.PathSegments.Add(*InSoftObjectPath.ToString());
		return Path;
	}

	FOutlinerItemPath AppendSegment(const FName& InSegment) const
	{
		FOutlinerItemPath Path = *this;
		Path.PathSegments.Add(InSegment);
		return Path;
	}

	FOutlinerItemPath RemoveSegment() const
	{
		FOutlinerItemPath Path = *this;
		if (Path.PathSegments.Num())
		{
			Path.PathSegments.Pop();
		}			
		return Path;
	}

	friend uint32 GetTypeHash(const FOutlinerItemPath& Path)
	{
		uint32 Hash = INDEX_NONE;

		if (Path.PathSegments.Num() == 1)
		{
			Hash = GetTypeHash(Path.PathSegments[0]);
		}
		else if (Path.PathSegments.Num() > 1)
		{
			Hash = GetTypeHash(Path.PathSegments[0]);
			for (int32 Index = 1; Index < Path.PathSegments.Num(); ++Index)
			{
				Hash = HashCombine(Hash, GetTypeHash(Path.PathSegments[Index]));				
			}
			return Hash;
		}
		
		return Hash;
	}
};

USTRUCT()
struct FWorkspaceOutlinerItemExport
{
	GENERATED_BODY()

	FWorkspaceOutlinerItemExport() = default;

	FWorkspaceOutlinerItemExport(const FName InIdentifier, const FSoftObjectPath& InObjectPath)
	{
		Path.PathSegments.Add(*InObjectPath.ToString());
		Path.PathSegments.Add(InIdentifier);
	}

	FWorkspaceOutlinerItemExport(const FName InIdentifier, const FWorkspaceOutlinerItemExport& InParent) 
		: Path(InParent.Path.AppendSegment(InIdentifier))
	{
	}

	FWorkspaceOutlinerItemExport(const FName InIdentifier, const FWorkspaceOutlinerItemExport& InParent, const TInstancedStruct<FWorkspaceOutlinerItemData>& InData) 
		: Path(InParent.Path.AppendSegment(InIdentifier))
		, Data(InData)
	{
	}

	bool operator==(const FWorkspaceOutlinerItemExport& Other) const
	{
		return this->GetFullPath().Equals(Other.GetFullPath());
	}

protected:
	/** Full 'path' to item this instance represents, expected to take form of AssetPath follow by a set of identifier names */
	UPROPERTY()
	FOutlinerItemPath Path;

	UPROPERTY()
	TInstancedStruct<FWorkspaceOutlinerItemData> Data;
public:
	FName GetIdentifier() const
	{
		// Path needs at least two segments to contain a valid identifier
		if(Path.PathSegments.Num() > 1)
		{
			return Path.PathSegments.Last();	
		}

		return NAME_None;
	}
	
	FName GetParentIdentifier() const
	{
		// Path needs at least three segments to contain a valid _parent_ identifier
		const int32 NumSegments = Path.PathSegments.Num();
		if (NumSegments > 2)
		{
			return Path.PathSegments[FMath::Max(NumSegments - 2, 0)];
		}

		return NAME_None;
	}

	template<typename AssetClass>
	AssetClass* GetFirstAssetOfType() const
	{
		if(Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath.IsAsset())
				{
					if (UObject* Object = ObjectPath.TryLoad())
					{
						if (AssetClass* TypedObject = Cast<AssetClass>(Object))
						{
							return TypedObject;
						}
					}
				}
			}
		}

		return nullptr;
	}

	template<typename AssetClass>
	AssetClass* GetFirstObjectOfType() const
	{
		if(Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid())
				{
					if (!ObjectPath.IsSubobject() && !GetIdentifier().IsNone() && ObjectPath.GetAssetName() != GetIdentifier())
					{
						// Check if Identifier points to a valid subobject
						ObjectPath.SetSubPathString(GetIdentifier().ToUtf8String());
						if (UObject* Object = ObjectPath.TryLoad())
						{
							if (AssetClass* TypedObject = Cast<AssetClass>(Object))
							{
								return TypedObject;
							}
						}
						ObjectPath.SetSubPathString(FUtf8String());
					}

					if (UObject* Object = ObjectPath.TryLoad())
					{
						if (AssetClass* TypedObject = Cast<AssetClass>(Object))
						{
							return TypedObject;
						}
					}
				}
			}
		}

		return nullptr;
	}

	// Returns the first FSoftObjectPath found in segments, starting from the end. e.g. "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return SoftObjectPathTwo
	FSoftObjectPath GetFirstAssetPath() const
	{
		// Path needs at least one segment to contain a (potentially) valid asset path
		if(Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath.IsAsset())
				{
					return ObjectPath;	
				}
			}
			
			return FSoftObjectPath(Path.PathSegments[0].ToString());
		}
		
		return FSoftObjectPath();
	}

	// Tries to get the first valid path to an object - taking into account the Identifier might point to a subobject
	// To verify validity, it may try to load the object if bForceLoad == true (default)
	FSoftObjectPath GetFirstValidObjectPath(bool bForceLoad = true) const
	{
		if (Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid())
				{
					if (ObjectPath.IsSubobject())
					{
						// If the path is valid and already includes a subobject we can return directly
						return ObjectPath;
					}
					else if (ObjectPath.GetAssetName() != GetIdentifier())
					{
						// Otherwise check if adding the identifier as a subobject results in a valid object.
						ObjectPath.SetSubPathString(GetIdentifier().ToUtf8String());

						UObject* TestObject = ObjectPath.ResolveObject();
						TestObject = TestObject ? TestObject : (bForceLoad ? ObjectPath.TryLoad() : nullptr);
						if (TestObject)
						{
							return ObjectPath;
						}

						ObjectPath.SetSubPathString(FUtf8String());
					}
					return ObjectPath;
				}
			}
		}

		return FSoftObjectPath();
	}

	// Returns the first path segment as a FSoftObjectPath. e.g:
	//  - "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return SoftObjectPath	
	//  - "Foo" - "SoftObjectPath" - "SoftObjectPathTwo" - "Bar" will return FSoftObjectPath()
	FSoftObjectPath GetTopLevelAssetPath() const
	{
		// Path needs at least one segment to contain a (potentially) valid asset path
		if(Path.PathSegments.Num() > 0)
		{
			return FSoftObjectPath(Path.PathSegments[0].ToString());
		}
		
		return FSoftObjectPath();
	}

	// Returns all valid FSoftObjectPaths found in path segments, starting from the end. e.g:
	//  - "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return "SoftObjectPathTwo", "SoftObjectPath"
	//  - "Foo" - "SoftObjectPath" - "SoftObjectPathTwo" - "Bar" will also return "SoftObjectPathTwo", "SoftObjectPath"	
	void GetAssetPaths(TArray<FSoftObjectPath>& OutAssetPaths) const
	{
		if(Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath.IsAsset())
				{
					OutAssetPaths.Add(ObjectPath);
				}
			}
		}
	}

	// Returns all valid UObject-based assets found, and (optionally) loaded, in path segments, starting from the end. e.g:
	//  - "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return "SoftObjectPath" and "SoftObjectPathTwo" if their class matches AssetClass
	template<typename AssetClass>
	void GetAssetsOfType(TArray<AssetClass*>& OutAssets, bool bForceLoad = true) const
	{
		if(Path.PathSegments.Num() > 0)
		{
			for (int32 SegmentIndex = Path.PathSegments.Num() - 1; SegmentIndex >= 0; SegmentIndex--)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath.IsAsset())
				{
					AssetClass* Asset = Cast<AssetClass>(ObjectPath.ResolveObject());
					Asset = Asset ? Asset : (bForceLoad ? Cast<AssetClass>(ObjectPath.TryLoad()) : nullptr);
					
					if (Asset)
					{
						OutAssets.Add(Asset);
					}
				}
			}
		}
	}
	
	// Returns all valid FWorkspaceOutlinerItemExports found in path segments, starting from the end. e.g:
	//  - "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return "SoftObjectPath" - "Bar" , "SoftObjectPathTwo" - "Foo"
	void GetExports(TArray<FWorkspaceOutlinerItemExport>& OutExports) const
	{
		if(Path.PathSegments.Num() > 0)
		{
			bool bFirstExport = true;
			const int32 NumSegments = Path.PathSegments.Num();
			for (int32 SegmentIndex = NumSegments - 1; SegmentIndex >= 0; SegmentIndex--)
			{				
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath.IsAsset())
				{
					FWorkspaceOutlinerItemExport Export;
					Export.Path.PathSegments.Append(&Path.PathSegments[0], bFirstExport ? NumSegments : SegmentIndex + 1);
					OutExports.Add(Export);
					bFirstExport = false;
				}
			}
		}
	}
			
	// Returns relative export to the provided InAssetPath
	//  - "SoftObjectPath" - "Foo" - "SoftObjectPathTwo" - "Bar" will return "SoftObjectPath" - "Foo" when passed "SoftObjectPathTwo" or an empty export when passed "SoftObjectPath"
	FWorkspaceOutlinerItemExport GetRelativeAssetExport(const FSoftObjectPath& InAssetPath) const
	{
		FWorkspaceOutlinerItemExport Export;
		
		if(Path.PathSegments.Num() > 0)
		{
			const int32 NumSegments = Path.PathSegments.Num();
			for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; SegmentIndex++)
			{
				FSoftObjectPath ObjectPath = Path.PathSegments[SegmentIndex].ToString();
				if (ObjectPath.IsValid() && ObjectPath == InAssetPath && SegmentIndex != 0)
				{
					Export.Path.PathSegments.Append(&Path.PathSegments[0], SegmentIndex);
					break;
				}
			}
		}
		
		return Export;
	}

	FString GetFullPath() const
	{
		FString FullPath;
		for (int32 SegmentIndex = 0; SegmentIndex < Path.PathSegments.Num(); SegmentIndex++)
		{
			FullPath.Append(Path.PathSegments[SegmentIndex].ToString());
		}

		return FullPath;
	}
	
	// Remove identifier segment to retrieve parent path hash
	uint32 GetParentHash() const { return GetTypeHash(Path.RemoveSegment()); }

	// Returns whether Data has any instanced struct setup
	bool HasData() const { return Data.IsValid(); }
	
	const TInstancedStruct<FWorkspaceOutlinerItemData>& GetData() const { return Data; }
	TInstancedStruct<FWorkspaceOutlinerItemData>& GetData() { return Data; }	

	friend uint32 GetTypeHash(const FWorkspaceOutlinerItemExport& Export)
	{
		return GetTypeHash(Export.Path);
	}

	// Returns the inner ReferredExport from the item data only valid for asset references, otherwise will return *this
	WORKSPACEEDITOR_API FWorkspaceOutlinerItemExport& GetResolvedExport();	
	WORKSPACEEDITOR_API const FWorkspaceOutlinerItemExport& GetResolvedExport() const;
};

USTRUCT()
struct FWorkspaceOutlinerAssetReferenceItemData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()

	FWorkspaceOutlinerAssetReferenceItemData() = default;

	UPROPERTY()
	FSoftObjectPath ReferredObjectPath;

	UPROPERTY()
	FWorkspaceOutlinerItemExport ReferredExport;

	// Whether this asset reference should be expanded in the Workspace outliner
	UPROPERTY()
	bool bShouldExpandReference = true;

	UPROPERTY()
	bool bRecursiveReference = false;

	static bool IsAssetReference(const FWorkspaceOutlinerItemExport& InExport)
	{
		return InExport.HasData() && InExport.GetData().GetScriptStruct() == FWorkspaceOutlinerAssetReferenceItemData::StaticStruct();
	}
};

USTRUCT()
struct FWorkspaceOutlinerGroupItemData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()

	FWorkspaceOutlinerGroupItemData() = default;

	UPROPERTY()
	FString GroupName;

	UPROPERTY()
	FSlateBrush GroupIcon;

	static bool IsGroupItem(const FWorkspaceOutlinerItemExport& InExport)
	{
		return InExport.HasData() && InExport.GetData().GetScriptStruct() == FWorkspaceOutlinerGroupItemData::StaticStruct();
	}
};

namespace UE::Workspace
{

static const FLazyName ExportsWorkspaceItemsRegistryTag = TEXT("WorkspaceItemExports");

}

USTRUCT()
struct FWorkspaceOutlinerItemExports
{
	GENERATED_BODY()

	FWorkspaceOutlinerItemExports() = default;

	UPROPERTY()
	TArray<FWorkspaceOutlinerItemExport> Exports;
};
