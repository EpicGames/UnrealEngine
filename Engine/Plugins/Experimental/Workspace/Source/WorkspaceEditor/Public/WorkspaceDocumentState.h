// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceAssetRegistryInfo.h"
#include "WorkspaceDocumentState.generated.h"

// Base struct used to persist workspace document state
USTRUCT()
struct FWorkspaceDocumentState
{
	GENERATED_BODY()

	FWorkspaceDocumentState() = default;

	explicit FWorkspaceDocumentState(const UObject* InObject, const FWorkspaceOutlinerItemExport& InExport)
		: Object(InObject), Export(InExport)
	{}

	UPROPERTY()
	FSoftObjectPath Object;

	// Export provided when opening object
	UPROPERTY()
	FWorkspaceOutlinerItemExport Export;

	friend bool operator==(const FWorkspaceDocumentState& LHS, const FWorkspaceDocumentState& RHS)
	{
		return LHS.Object == RHS.Object && GetTypeHash(LHS.Export) == GetTypeHash(RHS.Export);
	}
};
