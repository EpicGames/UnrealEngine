// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ClassIconFinder.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "AssetRegistry/AssetData.h"

class FWorkspaceDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWorkspaceDragDropOp, FDecoratedDragDropOp)

	/** Root-level Asset data workspace entries */
	TArray<FAssetData> AssetDatas;

	void Init(const TArray<FAssetData>& InAssetDatas) 
	{
		AssetDatas = InAssetDatas;

		if(AssetDatas.Num() == 0)
		{
			CurrentHoverText = NSLOCTEXT("FWorkspaceDragDropOp", "None", "None");
		}
		else 
		{
			const FText FirstAssetText = FText::FromName(AssetDatas[0].AssetName);
			const FText ExtraAssetsText = FText::Format(NSLOCTEXT("FWorkspaceDragDropOp", "TooltipAssetsFormat", " and {0} {1}|plural(one=other, other=others)"), FText::AsNumber(AssetDatas.Num() - 1), AssetDatas.Num() - 1);
						
			CurrentHoverText = FText::Format(NSLOCTEXT("FWorkspaceDragDropOp", "TooltipFormat", "{0}{1}"), FirstAssetText, AssetDatas.Num() > 1 ? ExtraAssetsText : FText::GetEmpty() );
		}

		CurrentIconBrush = FClassIconFinder::FindThumbnailForClass(UObject::StaticClass());

		SetupDefaults();
	}
};
