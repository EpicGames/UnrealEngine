// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "WorkspaceFactory.generated.h"

#define UE_API WORKSPACEEDITOR_API

class UWorkspaceSchema;

namespace UE::Workspace
{
	struct FUtils;
	class FWorkspaceEditorModule;
	class SWorkspacePicker;
}

UCLASS(MinimalAPI, BlueprintType)
class UWorkspaceFactory : public UFactory
{
	GENERATED_BODY()

protected:
	UE_API UWorkspaceFactory();

	// Set the schema class for workspaces produced with this factory
	void SetSchemaClass(TSubclassOf<UWorkspaceSchema> InSchemaClass) { SchemaClass = InSchemaClass; }

private:
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override
	{
		return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
	}

	UPROPERTY()
	TSubclassOf<UWorkspaceSchema> SchemaClass;

	friend class UE::Workspace::FWorkspaceEditorModule;
	friend class UE::Workspace::SWorkspacePicker;
};

#undef UE_API
