// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceAssetRegistryInfo.h"
#include "SceneOutlinerFwd.h"

#include "WorkspaceItemMenuContext.generated.h"

class FUICommandList;

UCLASS(MinimalAPI)
class UWorkspaceItemMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TArray<FWorkspaceOutlinerItemExport> SelectedExports;
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	TWeakPtr<FUICommandList> WeakCommandList;
};
