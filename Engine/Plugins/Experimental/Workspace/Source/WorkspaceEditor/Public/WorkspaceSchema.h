// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceViewportSceneDescription.h"

#include "WorkspaceSchema.generated.h"

class UWorkspace;
struct FInstancedStruct;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

// Workspace schema to configure workspace assets to specific use cases
UCLASS(MinimalAPI, Abstract)
class UWorkspaceSchema : public UObject
{
	GENERATED_BODY()

public:
	// Get the name to display for workspace assets that use this schema
	virtual FText GetDisplayName() const PURE_VIRTUAL(UWorkspaceSchema::GetDisplayName, return FText::GetEmpty(); )

	// Get the asset types that are supported by this workspace. If this is empty, all assets are assumed to be supported
	virtual TConstArrayView<FTopLevelAssetPath> GetSupportedAssetClassPaths() const PURE_VIRTUAL(UWorkspaceSchema::GetSupportedAssetClassPaths, return TConstArrayView<FTopLevelAssetPath>(); )

	// Called prior to saving workspace state to populate an instanced struct to hold user-defined persistent workspace state.
	virtual void OnSaveWorkspaceState(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, FInstancedStruct& OutWorkspaceState) const {}

	// Called after loading workspace state. InWorkspaceState is an instanced struct that holds user-defined persistent workspace state. Struct is not guaranteed to be valid.
	virtual void OnLoadWorkspaceState(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, const FInstancedStruct& InWorkspaceState) const {}

	// Whether workspaces using this schema can spawn viewport tabs.
	virtual bool SupportsViewport() const { return false; };

	// Create a new object containing the required data to construct a scene for this type of workspace
	virtual TObjectPtr<UWorkspaceViewportSceneDescription> CreateSceneDescription() const { return NewObject<UWorkspaceViewportSceneDescription>(); }
	
	// Whether workspaces using this schema can add more assets to the workspace besides the root-level asset
	virtual bool SupportsMultipleAssets() const { return false; }
	
	// Whether workspaces using this schema has a root-level asset that always exists in the workspace
	virtual bool HasPersistentRootAsset() const { return false; }
	
	// Whether workspaces using this schema can be saved as an asset
	virtual bool CanSaveWorkspace() const { return false; }
};
