// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

namespace UE::Workspace
{
struct FTabPayload_WorkspaceDocument : FTabPayload
{
	static const FName DocumentPayloadName;
	
	static TSharedRef<FTabPayload_WorkspaceDocument> Make(const UObject* DocumentID, const FWorkspaceOutlinerItemExport& InExport)
	{
		check(DocumentID);
		return MakeShareable(new FTabPayload_WorkspaceDocument(const_cast<UObject*>(DocumentID), InExport));
	}

	static TSharedRef<FTabPayload_WorkspaceDocument> Make(const UObject* DocumentID)
	{
		return MakeShareable(new FTabPayload_WorkspaceDocument(const_cast<UObject*>(DocumentID), FWorkspaceOutlinerItemExport()));
	}

	template <typename CastType>
	static CastType* CastChecked(TSharedRef<FTabPayload> Payload)
	{
		check(Payload->PayloadType == FTabPayload_WorkspaceDocument::DocumentPayloadName);

		UObject* UntypedObject = StaticCastSharedRef<FTabPayload_WorkspaceDocument>(Payload)->DocumentID.Get(true);
		return ::CastChecked<CastType>(UntypedObject);
	}

	static const FWorkspaceOutlinerItemExport& GetExport(const TSharedRef<FTabPayload> Payload)
	{
		check(Payload->PayloadType == FTabPayload_WorkspaceDocument::DocumentPayloadName);
		return StaticCastSharedRef<FTabPayload_WorkspaceDocument>(Payload)->Export;
	}

	// Determine if another payload is the same as this one
	virtual bool IsEqual(const TSharedRef<FTabPayload>& OtherPayload) const override
	{
		if (OtherPayload->PayloadType == PayloadType)
		{
			return this->DocumentID.HasSameIndexAndSerialNumber(StaticCastSharedRef<FTabPayload_WorkspaceDocument>(OtherPayload)->DocumentID)
			// Replace with ==operator
			&& GetTypeHash(StaticCastSharedRef<FTabPayload_WorkspaceDocument>(OtherPayload)->Export) == GetTypeHash(Export);
		}

		return false;
	}

	virtual bool IsValid() const override
	{
		return DocumentID.IsValid() && (Export.GetFirstAssetPath().IsValid() || Export.GetFirstAssetPath().IsNull());
	}

	virtual ~FTabPayload_WorkspaceDocument() override {};
	
private:
	FTabPayload_WorkspaceDocument(UObject* InDocumentID, const FWorkspaceOutlinerItemExport& InExport)
		: FTabPayload(DocumentPayloadName)
		, DocumentID(InDocumentID)
		, Export(InExport)
	{
	}

private:
	TWeakObjectPtr<UObject> DocumentID;
	FWorkspaceOutlinerItemExport Export;
};
}