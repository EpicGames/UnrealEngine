// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "WorkspaceViewportSceneDescription.h"

class FAdvancedPreviewScene;
class UWorkspaceViewportSceneDescription;

namespace UE::Workspace
{
	class IWorkspaceViewportController
	{
	public:
		virtual ~IWorkspaceViewportController() = default;

		class FViewportContext : public FGCObject
		{
		public:
			FAdvancedPreviewScene* PreviewScene;
			TObjectPtr<UObject> OutlinerObject;
			TObjectPtr<UWorkspaceViewportSceneDescription> SceneDescription;
			
			// FGCObject interface
			virtual FString GetReferencerName() const override { return TEXT("IWorkspaceViewportController::FViewportContext"); }
			virtual void AddReferencedObjects(FReferenceCollector& Collector) override
			{
				Collector.AddReferencedObject(OutlinerObject);
				Collector.AddReferencedObject(SceneDescription);
			}
		};
		
		virtual void OnEnter(const FViewportContext& InContext) = 0;
		virtual void OnExit(FAdvancedPreviewScene* PreviewScene) = 0;
	};

	using FWorkspaceViewportControllerFactory = TFunctionRef<TUniquePtr<IWorkspaceViewportController>()>;
}
