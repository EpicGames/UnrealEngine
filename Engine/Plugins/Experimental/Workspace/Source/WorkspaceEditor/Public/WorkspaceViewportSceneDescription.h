// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WorkspaceViewportSceneDescription.generated.h"

#define UE_API WORKSPACEEDITOR_API

UCLASS(MinimalAPI)
class UWorkspaceViewportSceneDescription : public UObject
{
	GENERATED_BODY()

public:
	UE_API FSimpleMulticastDelegate& GetOnConfigChanged();

protected:
	UE_API void BroadcastOnConfigChanged();

private:
	FSimpleMulticastDelegate OnConfigChanged;
};

#undef UE_API
