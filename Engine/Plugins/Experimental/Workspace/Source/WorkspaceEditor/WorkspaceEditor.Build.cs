// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WorkspaceEditor : ModuleRules
	{
		public WorkspaceEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core", 
					"UnrealEd", 
				});

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"SceneOutliner", 
				});
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"AssetTools",
					"ApplicationCore",
					"SlateCore",
					"Slate",
					"InputCore",
					"PropertyEditor",
					"GraphEditor",
					"ToolWidgets",
					"ToolMenus",
					"AssetDefinition",
					"SourceControl", 
					"KismetWidgets",
					"DesktopWidgets",
					"Settings",
					"EditorWidgets",
					"Json",
					"JsonUtilities",
					"AssetRegistry",
					"ContentBrowser",
					"SceneOutliner",
					"EditorInteractiveToolsFramework",
					"InteractiveToolsFramework",
					"EditorFramework",
					"AdvancedPreviewScene"
				}
			);
		}
	}
}