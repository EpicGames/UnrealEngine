{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Workspace",
	"Description" : "Editor framework allowing multiple assets to be edited in a unified workspace UI",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins" :
	[
	],
	"Modules" :
	[
		{
			"Name" : "WorkspaceEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}
